// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class AdvancedPreviewScene : ModuleRules
{
    public AdvancedPreviewScene(ReadOnlyTargetRules Target) : base(Target)
    {
        PublicIncludePathModuleNames.AddRange(
            new string[] {
                "PropertyEditor",
            }
        );

        PrivateDependencyModuleNames.AddRange(
            new string[] {
				"CommonMenuExtensions",
                "Core",
                "CoreUObject",
                "Engine",
				"EditorInteractiveToolsFramework",
                "InputCore",
                "Slate",
                "SlateCore",
				"ToolMenus",
                "UnrealEd",
            }
        );
    }
}
