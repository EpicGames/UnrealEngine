// Copyright Epic Games, Inc. All Rights Reserved.

#include "AdvancedPreviewScene.h"
#include "UnrealClient.h"
#include "SceneInterface.h"

#include "Components/PostProcessComponent.h"
#include "Engine/Texture.h"
#include "Engine/TextureCube.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstanceConstant.h"
#include "UObject/Package.h"

#include "Editor/EditorPerProjectUserSettings.h"

#include "Components/DirectionalLightComponent.h"
#include "Components/SkyLightComponent.h"
#include "Components/StaticMeshComponent.h"
#include "Components/SphereReflectionCaptureComponent.h"

#include "AssetViewerSettings.h"
#include "EditorModeManager.h"
#include "EditorViewportClient.h"
#include "Tools/EdModeInteractiveToolsContext.h"

#include "ViewportInteractions/ViewportInteractionsBehaviorSource.h"
#include "ViewportInteractions/ViewportLightRotationInteraction.h"
#include "ViewportInteractions/ViewportSkyRotationInteraction.h"

#include "Engine/StaticMesh.h"
#include "AdvancedPreviewSceneCommands.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Editor.h"
#include "ShowFlagMenuCommands.h"
#include "Widgets/Notifications/SNotificationList.h"

#define LOCTEXT_NAMESPACE "AdvancedPreviewScene"

FAdvancedPreviewScene::FAdvancedPreviewScene(ConstructionValues CVS, float InFloorOffset)
	: FPreviewScene(CVS)
{
	DefaultSettings = UAssetViewerSettings::Get();
	check(DefaultSettings);

	RefreshDelegate = DefaultSettings->OnAssetViewerSettingsChanged().AddRaw(this, &FAdvancedPreviewScene::OnAssetViewerSettingsRefresh);

	const int32 SettingsProfileIndex = GetDefault<UEditorPerProjectUserSettings>()->AssetViewerProfileIndex;
	CurrentProfileIndex = DefaultSettings->Profiles.IsValidIndex(SettingsProfileIndex) ? SettingsProfileIndex : 0;
	ensureMsgf(DefaultSettings->Profiles.IsValidIndex(CurrentProfileIndex), TEXT("Invalid default settings pointer or current profile index"));
	FPreviewSceneProfile& Profile = DefaultSettings->Profiles[CurrentProfileIndex];
	Profile.LoadProfileObjects();

	const FTransform Transform(FRotator(0, 0, 0), FVector(0, 0, 0), FVector(1));

	// Always set up sky light using the set cube map texture, reusing the sky light from PreviewScene class
	SetSkyCubemap(Profile.EnvironmentCubeMap.Get());
	SetSkyBrightness(Profile.SkyLightIntensity);
	
	// Large scale to prevent sphere from clipping
	const FTransform SphereTransform(FRotator(0, 0, 0), FVector(0, 0, 0), FVector(2000));
	SkyComponent = NewObject<UStaticMeshComponent>(GetTransientPackage());

	// Set up sky sphere showing the same cube map as used by the sky light
	UStaticMesh* SkySphere = LoadObject<UStaticMesh>(NULL, TEXT("/Engine/EditorMeshes/AssetViewer/Sphere_inversenormals.Sphere_inversenormals"), NULL, LOAD_None, NULL);
	check(SkySphere);
	SkyComponent->SetStaticMesh(SkySphere);
	SkyComponent->SetCollisionEnabled(ECollisionEnabled::NoCollision);
	SkyComponent->CastShadow = false;
	SkyComponent->bCastDynamicShadow = false;

	UMaterial* SkyMaterial = LoadObject<UMaterial>(NULL, TEXT("/Engine/EditorMaterials/AssetViewer/M_SkyBox.M_SkyBox"), NULL, LOAD_None, NULL);
	check(SkyMaterial);

	InstancedSkyMaterial = NewObject<UMaterialInstanceConstant>(GetTransientPackage());
	InstancedSkyMaterial->Parent = SkyMaterial;		

	UTextureCube* DefaultTexture = LoadObject<UTextureCube>(NULL, TEXT("/Engine/MapTemplates/Sky/SunsetAmbientCubemap.SunsetAmbientCubemap"));
    
	InstancedSkyMaterial->SetTextureParameterValueEditorOnly(FName("SkyBox"), ( Profile.EnvironmentCubeMap.Get() != nullptr ) ? Profile.EnvironmentCubeMap.Get() : DefaultTexture );
	InstancedSkyMaterial->SetScalarParameterValueEditorOnly(FName("CubemapRotation"), Profile.LightingRigRotation / 360.0f);
	InstancedSkyMaterial->SetScalarParameterValueEditorOnly(FName("Intensity"), Profile.SkyLightIntensity);
	InstancedSkyMaterial->PostLoad();
	SkyComponent->SetMaterial(0, InstancedSkyMaterial);
	AddComponent(SkyComponent, SphereTransform);
	
	PostProcessComponent = NewObject<UPostProcessComponent>();
	PostProcessComponent->Settings = Profile.PostProcessingSettings;
	PostProcessComponent->bUnbound = true;
	AddComponent(PostProcessComponent, Transform);

	FloorMeshComponent = NewObject<UStaticMeshComponent>(GetTransientPackage());
	FloorMeshComponent->bSelectable = false;
	UStaticMesh* FloorMesh = Profile.GetEnvironmentFloorMesh();
	SetFloorMesh(FloorMesh);

	UMaterialInterface* FloorMaterial = Profile.GetEnvironmentFloorMaterial();
	SetFloorMaterial(FloorMaterial);

	FTransform FloorTransform(Profile.EnvironmentFloorRotation, FVector(0, 0, -(InFloorOffset)), Profile.EnvironmentFloorScale);
	AddComponent(FloorMeshComponent, FloorTransform);

	SetLightDirection(Profile.DirectionalLightRotation);

	bRotateLighting = Profile.bRotateLightingRig;
	CurrentRotationSpeed = Profile.RotationSpeed;
	bSkyChanged = false;

	BindCommands();

	// since advance preview scenes are used in conjunction with material/mesh editors we should match their feature level with the global one
	if (GIsEditor && GEditor != nullptr && GWorld != nullptr && GWorld->Scene != nullptr)
	{
		PreviewWorld->ShaderPlatformChanged(GWorld->Scene->GetShaderPlatform());
	}

	PreviousRotation = Profile.LightingRigRotation;
	UILightingRigRotationDelta = 0.0f;

	// Ensure visibility properties etc are set (note that UpdateScene doesn't get called continuously)
	UpdateScene(Profile);
}

FAdvancedPreviewScene::~FAdvancedPreviewScene()
{
	if (UObjectInitialized())
	{
		DefaultSettings = UAssetViewerSettings::Get();
		if (DefaultSettings)
		{
			DefaultSettings->OnAssetViewerSettingsChanged().Remove(RefreshDelegate);
		}
	}
}

void FAdvancedPreviewScene::UpdateScene(FPreviewSceneProfile& Profile, bool bUpdateSkyLight /*= true*/, bool bUpdateEnvironment  /*= true*/, bool bUpdatePostProcessing /*= true*/, bool bUpdateDirectionalLight /*= true*/)
{
	if (bUpdateSkyLight && SkyLight)
	{
		// Threshold to ensure we only update the intensity if it is going to make a difference
		if (!FMath::IsNearlyEqual(SkyLight->Intensity, Profile.SkyLightIntensity, 0.05f))
		{
			static const FName IntensityName("Intensity");
			SetSkyBrightness(Profile.SkyLightIntensity);
			
			InstancedSkyMaterial->SetScalarParameterValueEditorOnly(IntensityName, Profile.SkyLightIntensity);
			bSkyChanged = true;
		}
	}

	if (bUpdateEnvironment)
	{
		if (SkyLight)
		{
			static const FName SkyBoxName("SkyBox");
			static const FName CubeMapRotationName("CubemapRotation");

			UTextureCube* EnvironmentTexture = Profile.EnvironmentCubeMap.LoadSynchronous();
			UTexture* Texture = EnvironmentTexture;
			InstancedSkyMaterial->GetTextureParameterValue(SkyBoxName, Texture);

			if (Texture != EnvironmentTexture)
			{
				InstancedSkyMaterial->SetTextureParameterValueEditorOnly(SkyBoxName, EnvironmentTexture);
				SetSkyCubemap(EnvironmentTexture);
				bSkyChanged = true;
			}
		
			static const float OneOver360 = 1.0f / 360.0f;
			float Rotation = Profile.LightingRigRotation;
			InstancedSkyMaterial->GetScalarParameterValue(CubeMapRotationName, Rotation);
			const bool bLightRighRotationChanged = !FMath::IsNearlyEqual(Rotation, Profile.LightingRigRotation, 0.05f);

			// We also test any change of light direction because the default profile light orientation might not match the LightingRigRotation.
			FRotator LightDir = GetLightDirection();
			const bool bLightDirChanged = LightDir != Profile.DirectionalLightRotation;

			if (bLightRighRotationChanged || bLightDirChanged)
			{			
				InstancedSkyMaterial->SetScalarParameterValueEditorOnly(CubeMapRotationName, Profile.LightingRigRotation * OneOver360);

				// NOTE: this code is only executed when the UI rig rotation angle is interacted with.
				// the l+mouse shortcut and the rig smooth rotation are handled at other places for the directional light
				LightDir.Yaw += UILightingRigRotationDelta;
				UILightingRigRotationDelta = 0;
				SetLightDirection(LightDir);
				DefaultSettings->Profiles[CurrentProfileIndex].DirectionalLightRotation = LightDir;

				SkyLight->SourceCubemapAngle = Profile.LightingRigRotation;
				bSkyChanged = true;
			}
		}

		if (FloorMeshComponent)
		{
			UStaticMesh* FloorMesh = Profile.GetEnvironmentFloorMesh();
			SetFloorMesh(FloorMesh);
			FloorMeshComponent->SetRelativeScale3D(Profile.EnvironmentFloorScale);
			FloorMeshComponent->SetRelativeRotation(Profile.EnvironmentFloorRotation);
			UMaterialInterface* FloorMaterial = Profile.GetEnvironmentFloorMaterial();
			for (int32 Index = 0; Index < FloorMeshComponent->GetNumMaterials(); Index++)
			{
				FloorMeshComponent->SetMaterial(Index, FloorMaterial);
			}
		}
	}

	if (bUpdatePostProcessing)
	{
		PostProcessComponent->Settings = Profile.PostProcessingSettings;
		PostProcessComponent->bEnabled = Profile.bPostProcessingEnabled;
		bPostProcessing = Profile.bPostProcessingEnabled;
	}

	if (bUpdateDirectionalLight && DirectionalLight)
	{
		if (!FMath::IsNearlyEqual(DirectionalLight->Intensity, Profile.DirectionalLightIntensity, 0.05f))
		{
			DirectionalLight->SetIntensity(Profile.DirectionalLightIntensity);
		}
		DirectionalLight->SetLightColor(Profile.DirectionalLightColor);
	}

	SkyComponent->SetVisibility(Profile.bShowEnvironment, true);

	if (SkyLight)
	{
		SkyLight->SetVisibility(Profile.bUseSkyLighting, true);
	}
	FloorMeshComponent->SetVisibility(Profile.bShowFloor, true);

	bRotateLighting = Profile.bRotateLightingRig;
	CurrentRotationSpeed = Profile.RotationSpeed;
}

FLinearColor FAdvancedPreviewScene::GetBackgroundColor() const
{
	FLinearColor Color = DefaultSettings->Profiles[CurrentProfileIndex].EnvironmentColor;
	return Color * DefaultSettings->Profiles[CurrentProfileIndex].EnvironmentIntensity;
}

float FAdvancedPreviewScene::GetFloorOffset() const
{
	return -(float)FloorMeshComponent->GetRelativeLocation().Z;
}

void FAdvancedPreviewScene::SetFloorOffset(const float InFloorOffset)
{
	FloorMeshComponent->SetRelativeLocation(FVector(0, 0, -(InFloorOffset)));
}

void FAdvancedPreviewScene::SetProfileIndex(const int32 InProfileIndex)
{
	if (CurrentProfileIndex == InProfileIndex)
	{
		return;
	}

	CurrentProfileIndex = InProfileIndex;
	DefaultSettings->Profiles[CurrentProfileIndex].LoadProfileObjects();
	SetLightDirection(DefaultSettings->Profiles[CurrentProfileIndex].DirectionalLightRotation);
	PreviousRotation = DefaultSettings->Profiles[CurrentProfileIndex].LightingRigRotation;
	UILightingRigRotationDelta = 0.0f;

	UpdateScene(DefaultSettings->Profiles[CurrentProfileIndex]);
	DefaultSettings->OnAssetViewerSettingsChanged().Broadcast(NAME_None);
}

FPreviewSceneProfile* FAdvancedPreviewScene::GetCurrentProfile() const
{
	if (DefaultSettings->Profiles.IsValidIndex(CurrentProfileIndex))
	{
		return &DefaultSettings->Profiles[CurrentProfileIndex];
	}
	return nullptr;
}

void FAdvancedPreviewScene::Tick(float DeltaTime)
{
	checkf(DefaultSettings && DefaultSettings->Profiles.IsValidIndex(CurrentProfileIndex), TEXT("Invalid default settings pointer or current profile index"));
	UpdateCaptureContents();

	FPreviewSceneProfile& Profile = DefaultSettings->Profiles[CurrentProfileIndex];
	if (Profile.bRotateLightingRig)
	{
		CurrentRotationSpeed = Profile.RotationSpeed;
		Profile.LightingRigRotation = FMath::Fmod(FMath::Max(FMath::Min(Profile.LightingRigRotation + (CurrentRotationSpeed * DeltaTime), 360.0f), 0.0f), 360.0f);

		FRotator LightDir = GetLightDirection();
		LightDir.Yaw += DeltaTime * -CurrentRotationSpeed;
		SetLightDirection(LightDir);		
		DefaultSettings->Profiles[CurrentProfileIndex].DirectionalLightRotation = LightDir;
	}

	if (!FMath::IsNearlyEqual(PreviousRotation, Profile.LightingRigRotation, 0.05f))
	{		
		SkyLight->SourceCubemapAngle = Profile.LightingRigRotation;
		SkyLight->SetCaptureIsDirty();
		SkyLight->MarkRenderStateDirty();
		SkyLight->UpdateSkyCaptureContents(PreviewWorld);

		InstancedSkyMaterial->SetScalarParameterValueEditorOnly(FName("CubemapRotation"), Profile.LightingRigRotation / 360.0f);
		InstancedSkyMaterial->PostEditChange();

		const bool bInsideTick = true;
		UReflectionCaptureComponent::UpdateReflectionCaptureContents(PreviewWorld, nullptr, false, false, bInsideTick);
		PreviewWorld->UpdateAllSkyCaptures();

		PreviousRotation = Profile.LightingRigRotation;
		UILightingRigRotationDelta = 0.0f;
	}

	// Update the sky every tick rather than every mouse move (UpdateScene call)
	if (bSkyChanged)
	{
		SkyLight->SetCaptureIsDirty();
		SkyLight->MarkRenderStateDirty();
		SkyLight->UpdateSkyCaptureContents(PreviewWorld);

		InstancedSkyMaterial->PostEditChange();
		bSkyChanged = false;
	}
}

TStatId FAdvancedPreviewScene::GetStatId() const
{
	return TStatId();
}

const bool FAdvancedPreviewScene::HandleViewportInput(FViewport* InViewport, FInputDeviceId DeviceId, FKey Key, float Delta, float DeltaTime, int32 NumSamples, bool bGamepad)
{
	bool bResult = false;
	const bool bMouseButtonDown = InViewport->KeyState(EKeys::LeftMouseButton) || InViewport->KeyState(EKeys::MiddleMouseButton) || InViewport->KeyState(EKeys::RightMouseButton);

	const bool bSkyMove = InViewport->KeyState(EKeys::K);
	const bool bLightMoveDown = InViewport->KeyState(EKeys::L);

	// Look at which axis is being dragged and by how much
	const float DragX = (Key == EKeys::MouseX) ? Delta : 0.f;
	const float DragY = (Key == EKeys::MouseY) ? Delta : 0.f;

	// Skip legacy input handling when ITF viewport interactions are enabled — the interactions handle it.
	bool bUsesViewportInteractions = UsesViewportInteractions(static_cast<FEditorViewportClient*>(InViewport->GetClient()));

	if (!bUsesViewportInteractions)
	{
		// Move the sky around if K is down and the mouse has moved on the X-axis
		if (bSkyMove && bMouseButtonDown)
		{
			static constexpr float SkyRotationSpeed = 0.22f;
			float SkyRotation = GetSkyRotation();
			SkyRotation += -DragX * SkyRotationSpeed;
			SetSkyRotation(SkyRotation);
			bResult = true;
		}

		if (bLightMoveDown && (!FMath::IsNearlyZero(DragX) || !FMath::IsNearlyZero(DragY)))
		{
			// Save light rotation
			DefaultSettings->Profiles[CurrentProfileIndex].DirectionalLightRotation = GetLightDirection();
		}
	}

	return bResult;
}

const bool FAdvancedPreviewScene::HandleInputKey(const FInputKeyEventArgs& EventArgs)
{
	if (EventArgs.Event == IE_Pressed)
	{
		FModifierKeysState KeyState = FSlateApplication::Get().GetModifierKeys();
		if (UICommandList->ProcessCommandBindings(EventArgs.Key, KeyState, (EventArgs.Event == IE_Repeat))) //-V547
		{
			return true;
		}
	}

	return false;
}

bool FAdvancedPreviewScene::GetFloorVisibility() const
{
	if(FloorMeshComponent)
	{
		return FloorMeshComponent->GetVisibleFlag();
	}
	return false;
}

void FAdvancedPreviewScene::SetFloorVisibility(const bool bVisible, const bool bDirect)
{
	// If not direct set visibility in profile and refresh the scene
	if (!bDirect)
	{
		FName PropertyName("bShowFloor");

		FProperty* FloorProperty = FindFProperty<FProperty>(FPreviewSceneProfile::StaticStruct(), PropertyName);
		DefaultSettings->Profiles[CurrentProfileIndex].bShowFloor = bVisible;

		FPropertyChangedEvent PropertyEvent(FloorProperty);
		DefaultSettings->PostEditChangeProperty(PropertyEvent);
	}
	else
	{
		// Otherwise set visibility directly on the component
		FloorMeshComponent->SetVisibility(bVisible ? DefaultSettings->Profiles[CurrentProfileIndex].bShowFloor : bVisible);
	}
}

void FAdvancedPreviewScene::SetEnvironmentVisibility(const bool bVisible, const bool bDirect)
{
	// If not direct set visibility in profile and refresh the scene
	if (!bDirect)
	{
		FProperty* EnvironmentProperty = FindFProperty<FProperty>(FPreviewSceneProfile::StaticStruct(), GET_MEMBER_NAME_CHECKED(FPreviewSceneProfile, bShowEnvironment));
		DefaultSettings->Profiles[CurrentProfileIndex].bShowEnvironment = bVisible;

		FPropertyChangedEvent PropertyEvent(EnvironmentProperty);
		DefaultSettings->PostEditChangeProperty(PropertyEvent);
	}
	else
	{
		// Otherwise set visibility directly on the component
		SkyComponent->SetVisibility(bVisible ? DefaultSettings->Profiles[CurrentProfileIndex].bShowEnvironment : bVisible);
	}
}

const float FAdvancedPreviewScene::GetSkyRotation() const
{
	checkf(DefaultSettings && DefaultSettings->Profiles.IsValidIndex(CurrentProfileIndex), TEXT("Invalid default settings pointer or current profile index"));
	return DefaultSettings->Profiles[CurrentProfileIndex].LightingRigRotation;
}

void FAdvancedPreviewScene::SetSkyRotation(const float SkyRotation)
{
	checkf(DefaultSettings && DefaultSettings->Profiles.IsValidIndex(CurrentProfileIndex), TEXT("Invalid default settings pointer or current profile index"));

	float ClampedSkyRotation = FMath::Fmod(SkyRotation, 360.0f);
	// Clamp and wrap around sky rotation
	if (ClampedSkyRotation < 0.0f)
	{
		ClampedSkyRotation += 360.0f;
	}
	DefaultSettings->Profiles[CurrentProfileIndex].LightingRigRotation = ClampedSkyRotation;
}

const bool FAdvancedPreviewScene::IsUsingPostProcessing() const
{
	return bPostProcessing;
}

const int32 FAdvancedPreviewScene::GetCurrentProfileIndex() const
{
	return CurrentProfileIndex;
}

const UStaticMeshComponent* FAdvancedPreviewScene::GetFloorMeshComponent() const
{
	checkf(FloorMeshComponent != nullptr, TEXT("Invalid floor mesh component pointer"));
	return FloorMeshComponent;
}

void FAdvancedPreviewScene::BindCommands()
{
	UICommandList = CreateCommandList();

	const FAdvancedPreviewSceneCommands& Commands = FAdvancedPreviewSceneCommands::Get();

	UICommandList->MapAction(Commands.ToggleFloor,
		FExecuteAction::CreateRaw(this, &FAdvancedPreviewScene::HandleToggleFloor),
		FCanExecuteAction(),
		FIsActionChecked::CreateRaw(this, &FAdvancedPreviewScene::IsFloorEnabled)
	);

	UICommandList->MapAction(Commands.ToggleEnvironment,
		FExecuteAction::CreateRaw(this, &FAdvancedPreviewScene::HandleToggleEnvironment),
		FCanExecuteAction(),
		FIsActionChecked::CreateRaw(this, &FAdvancedPreviewScene::IsEnvironmentEnabled)
	);

	UICommandList->MapAction(Commands.TogglePostProcessing,
		FExecuteAction::CreateRaw(this, &FAdvancedPreviewScene::HandleTogglePostProcessing),
		FCanExecuteAction(),
		FIsActionChecked::CreateRaw(this, &FAdvancedPreviewScene::IsPostProcessingEnabled)
	);

	UICommandList->MapAction(Commands.NextProfile,
		FExecuteAction::CreateRaw(this, &FAdvancedPreviewScene::NavigateProfile, ENavigationDirection::Forward)
	);

	UICommandList->MapAction(Commands.PreviousProfile,
		FExecuteAction::CreateRaw(this, &FAdvancedPreviewScene::NavigateProfile, ENavigationDirection::Backward)
	);
}

TSharedRef<FUICommandList> FAdvancedPreviewScene::CreateCommandList()
{
	return MakeShared<FUICommandList>();
}

void FAdvancedPreviewScene::HandleToggleEnvironment()
{
	SetEnvironmentVisibility(!DefaultSettings->Profiles[CurrentProfileIndex].bShowEnvironment);
}

bool FAdvancedPreviewScene::IsEnvironmentEnabled() const
{
	return DefaultSettings->Profiles[CurrentProfileIndex].bShowEnvironment;
}

void FAdvancedPreviewScene::HandleToggleFloor()
{
	SetFloorVisibility(!DefaultSettings->Profiles[CurrentProfileIndex].bShowFloor);
}

bool FAdvancedPreviewScene::IsFloorEnabled() const
{
	return DefaultSettings->Profiles[CurrentProfileIndex].bShowFloor;
}

void FAdvancedPreviewScene::HandleToggleGrid()
{
	// DEPRECATED : Do nothing
}

bool FAdvancedPreviewScene::IsGridEnabled() const
{
	// DEPRECATED
	return false;
}

void FAdvancedPreviewScene::HandleTogglePostProcessing()
{
	FPreviewSceneProfile& Profile = DefaultSettings->Profiles[CurrentProfileIndex];
	Profile.bPostProcessingEnabled = !Profile.bPostProcessingEnabled;
	PostProcessComponent->bEnabled = Profile.bPostProcessingEnabled;
	bPostProcessing = Profile.bPostProcessingEnabled;
	
	FName PropertyName("bPostProcessingEnabled");
	FProperty* PostProcessingProperty = FindFProperty<FProperty>(FPreviewSceneProfile::StaticStruct(), PropertyName);
	FPropertyChangedEvent PropertyEvent(PostProcessingProperty);
	DefaultSettings->PostEditChangeProperty(PropertyEvent);	
}

bool FAdvancedPreviewScene::IsPostProcessingEnabled() const
{
	return DefaultSettings->Profiles[CurrentProfileIndex].bPostProcessingEnabled;
}

void FAdvancedPreviewScene::SetFloorMesh(UStaticMesh* InFloorMesh)
{
	if (FloorMeshComponent)
	{
		FloorMeshComponent->SetStaticMesh(InFloorMesh);
	}
}

void FAdvancedPreviewScene::SetFloorMaterial(UMaterialInterface* InFloorMaterial, int32 InIndex)
{
	if (FloorMeshComponent)
	{
		if (InIndex == INDEX_NONE)
		{
			for (int32 Index = 0; Index < FloorMeshComponent->GetNumMaterials(); Index++)
			{
				FloorMeshComponent->SetMaterial(Index, InFloorMaterial);
			}
		}
		else
		{
			FloorMeshComponent->SetMaterial(InIndex, InFloorMaterial);
		}
	}
}

void FAdvancedPreviewScene::SetFloorMeshRotation(const FRotator& InRotation)
{
	if (FloorMeshComponent)
	{
		FloorMeshComponent->SetRelativeRotation(InRotation);
	}
}

void FAdvancedPreviewScene::SetFloorMeshScale(const FVector& InScale)
{
	if (FloorMeshComponent)
	{
		FloorMeshComponent->SetRelativeScale3D(InScale);
	}
}

void FAdvancedPreviewScene::OnAssetViewerSettingsRefresh(const FName& InPropertyName)
{
	// If the profile was changed, update the current index and the scene.
	if (InPropertyName == GET_MEMBER_NAME_CHECKED(FPreviewSceneProfile, ProfileName))
	{
		CurrentProfileIndex = GetDefault<UEditorPerProjectUserSettings>()->AssetViewerProfileIndex;
		CurrentProfileIndex = DefaultSettings->Profiles.IsValidIndex(CurrentProfileIndex) ? CurrentProfileIndex : 0;
		PreviousRotation = DefaultSettings->Profiles[CurrentProfileIndex].LightingRigRotation;
		UILightingRigRotationDelta = 0.0f;

		UpdateScene(DefaultSettings->Profiles[CurrentProfileIndex]);
		
		OnProfileChangedDelegate.Broadcast(DefaultSettings->Profiles[CurrentProfileIndex], InPropertyName);
	}
	else if (DefaultSettings->Profiles.IsValidIndex(CurrentProfileIndex))
	{
		const bool bNameNone = InPropertyName == NAME_None;

		static const TSet<FName> EnvironmentPropertiesName
		{
			GET_MEMBER_NAME_CHECKED(FPreviewSceneProfile, EnvironmentCubeMap),
			GET_MEMBER_NAME_CHECKED(FPreviewSceneProfile, EnvironmentFloorMesh),
			GET_MEMBER_NAME_CHECKED(FPreviewSceneProfile, EnvironmentFloorRotation),
			GET_MEMBER_NAME_CHECKED(FPreviewSceneProfile, EnvironmentFloorScale),
			GET_MEMBER_NAME_CHECKED(FPreviewSceneProfile, EnvironmentFloorMaterial),
			GET_MEMBER_NAME_CHECKED(FPreviewSceneProfile, LightingRigRotation),
			GET_MEMBER_NAME_CHECKED(UAssetViewerSettings, Profiles)
		};

		const bool bUpdateEnvironment = EnvironmentPropertiesName.Contains(InPropertyName);
		const bool bUpdateSkyLight = bUpdateEnvironment || (InPropertyName == GET_MEMBER_NAME_CHECKED(FPreviewSceneProfile, SkyLightIntensity) || InPropertyName == GET_MEMBER_NAME_CHECKED(FPreviewSceneProfile, bUseSkyLighting) || (InPropertyName == GET_MEMBER_NAME_CHECKED(UAssetViewerSettings, Profiles)));
		const bool bUpdateDirectionalLight = (InPropertyName == GET_MEMBER_NAME_CHECKED(FPreviewSceneProfile, DirectionalLightIntensity)) || (InPropertyName == GET_MEMBER_NAME_CHECKED(FPreviewSceneProfile, DirectionalLightColor));
		const bool bUpdatePostProcessing = (InPropertyName == GET_MEMBER_NAME_CHECKED(FPreviewSceneProfile, PostProcessingSettings)) || (InPropertyName == GET_MEMBER_NAME_CHECKED(FPreviewSceneProfile, bPostProcessingEnabled));

		UILightingRigRotationDelta += PreviousRotation - DefaultSettings->Profiles[CurrentProfileIndex].LightingRigRotation;
		PreviousRotation = DefaultSettings->Profiles[CurrentProfileIndex].LightingRigRotation;

		UpdateScene(DefaultSettings->Profiles[CurrentProfileIndex], bUpdateSkyLight || bNameNone, bUpdateEnvironment || bNameNone, bUpdatePostProcessing || bNameNone, bUpdateDirectionalLight || bNameNone);
		
		OnProfileChangedDelegate.Broadcast(DefaultSettings->Profiles[CurrentProfileIndex], InPropertyName);
	}
}

bool FAdvancedPreviewScene::UsesViewportInteractions(const FEditorViewportClient* InViewportClient) const
{
	if (!ensure(InViewportClient))
	{
		return false;
	}

	FEditorModeTools* ModeTools = InViewportClient->GetModeTools();
	if (!ModeTools)
	{
		return false;
	}

	if (const UModeManagerInteractiveToolsContext* ITFContext = ModeTools->GetInteractiveToolsContext())
	{
		return ITFContext->UsesViewportInteractions();
	}

	return false;
}

void FAdvancedPreviewScene::NavigateProfile(ENavigationDirection InDirection)
{
	const bool bMoveForward = InDirection == ENavigationDirection::Forward;
	int32 NewProfileIndex = bMoveForward ? CurrentProfileIndex + 1 : CurrentProfileIndex - 1;
	if (!DefaultSettings->Profiles.IsValidIndex(NewProfileIndex))
	{
		NewProfileIndex = bMoveForward ? 0 : DefaultSettings->Profiles.Num() - 1;
	}
	// This is to avoid a desync and hit ensure in FPreviewProfileController
	GetMutableDefault<UEditorPerProjectUserSettings>()->AssetViewerProfileIndex = NewProfileIndex;
	SetProfileIndex(NewProfileIndex);
	OnProfileChangedDelegate.Broadcast(DefaultSettings->Profiles[NewProfileIndex], /** Proprety */NAME_None);

	// Notify user
	FSlateNotificationManager& NotificationManager = FSlateNotificationManager::Get();
	FNotificationInfo Info(FText::Format(LOCTEXT("NavigateAdvancedPreviewSceneProfile", "Applying scene profile: {0}"), FText::FromString(DefaultSettings->Profiles[NewProfileIndex].ProfileName)));
	Info.ExpireDuration = 4.f;
	NotificationManager.AddNotification(Info);
}

void FAdvancedPreviewScene::AddPreviewSceneInteractions(UViewportInteractionsBehaviorSource* InBehaviorSource)
{
	if (InBehaviorSource)
	{
		InBehaviorSource->AddInteraction<UViewportSkyRotationInteraction>();
		InBehaviorSource->AddInteraction<UViewportLightRotationInteraction>();
	}
}

#undef LOCTEXT_NAMESPACE
