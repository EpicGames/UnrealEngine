// Copyright Epic Games, Inc. All Rights Reserved.

#include "AdvancedPreviewSceneCommands.h"

#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UICommandInfo.h"
#include "InputCoreTypes.h"

#define LOCTEXT_NAMESPACE "AdvancedPreviewSceneCommands"

void FAdvancedPreviewSceneCommands::RegisterCommands()
{
	UI_COMMAND(ToggleEnvironment, "Toggle Environment", "Toggles Environment visibility", EUserInterfaceActionType::ToggleButton, FInputChord(EKeys::I));
	UI_COMMAND(ToggleFloor, "Toggle Floor", "Toggles floor visibility", EUserInterfaceActionType::ToggleButton, FInputChord(EKeys::O));
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UI_COMMAND(ToggleGrid, "Toggle Grid", "Toggles grid visibility", EUserInterfaceActionType::ToggleButton, FInputChord(EModifierKey::Alt | EModifierKey::Shift, EKeys::G));
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	UI_COMMAND(TogglePostProcessing, "Toggle Post Processing", "Toggles whether Post Processing is enabled", EUserInterfaceActionType::ToggleButton, FInputChord(EKeys::P));
	UI_COMMAND(NextProfile, "Next profile", "Navigate to the following profile", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Alt | EModifierKey::Shift, EKeys::Down));
	UI_COMMAND(PreviousProfile, "Previous profile", "Navigate to the preceding profile", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Alt | EModifierKey::Shift, EKeys::Up));
}

#undef LOCTEXT_NAMESPACE
