// Copyright Epic Games, Inc. All Rights Reserved.

#include "AdvancedPreviewSceneMenus.h"

#include "AdvancedPreviewScene.h"
#include "AdvancedPreviewSceneCommands.h"
#include "AssetViewerSettings.h"
#include "EditorViewportClient.h"
#include "ToolMenus.h"

#define LOCTEXT_NAMESPACE "AdvancedPreviewSceneMenus"

namespace UE::AdvancedPreviewScene::Menus
{

void ExtendAdvancedPreviewSceneSettings(FName InAssetViewerProfileSubmenuName, const FSettingsOptions& InSettingsOptions)
{
	UToolMenu* const Submenu = UToolMenus::Get()->ExtendMenu(InAssetViewerProfileSubmenuName);
	if (!Submenu)
	{
		return;
	}

	FToolMenuSection& Section = Submenu->FindOrAddSection(
		"PreviewSceneSettings",
		LOCTEXT("PreviewProfileSceneSettingsSectionLabel", "Preview Scene Options")
	);

	if (InSettingsOptions.bShowToggleEnvironment)
	{
		Section.AddMenuEntry(
			FAdvancedPreviewSceneCommands::Get().ToggleEnvironment,
			LOCTEXT("ToggleEnvironmentLabel", "Background"),
			LOCTEXT("ToggleEnvironmentTooltip", "Set the visibility of the preview scene's background.")
		);
	}
	
	if (InSettingsOptions.bShowToggleFloor)
	{
		Section.AddMenuEntry(
    		FAdvancedPreviewSceneCommands::Get().ToggleFloor,
    		LOCTEXT("ToggleFloorLabel", "Floor"),
    		LOCTEXT("ToggleFloorTooltip", "Set the visibility of the preview scene's floor.")
    	);
	}
	
	if (InSettingsOptions.bShowTogglePostProcessing)
	{
		Section.AddMenuEntry(
        	FAdvancedPreviewSceneCommands::Get().TogglePostProcessing,
        	LOCTEXT("TogglePostProcessingLabel", "Post Processing"),
        	LOCTEXT("TogglePostProcessingTooltip", "Set whether the preview scene includes post processing.")
        );
	}
}

}

namespace UE::AdvancedPreviewScene
{

void BindDefaultOnSettingsChangedHandler(const TSharedPtr<FAdvancedPreviewScene>& Scene, const TSharedPtr<FEditorViewportClient>& Client)
{
	Scene->OnProfileChanged().AddStatic(&DefaultOnSettingsChangedHandler, Client.ToWeakPtr());
	
	if (FPreviewSceneProfile* Profile = Scene->GetCurrentProfile())
	{
		// Do the initial sync of profile settings to client settings
		DefaultOnSettingsChangedHandler(*Profile, NAME_None, Client);
	}
}

void DefaultOnSettingsChangedHandler(const FPreviewSceneProfile& Profile, FName PropertyName, TWeakPtr<FEditorViewportClient> Client)
{
	if (TSharedPtr<FEditorViewportClient> ViewportClient = Client.Pin())
	{
		Profile.SetShowFlags(ViewportClient->EngineShowFlags);
		
		if (Profile.bRotateLightingRig && !ViewportClient->IsRealtime())
		{
			ViewportClient->SetRealtime(true);
		}
		
		ViewportClient->Invalidate();
	}
}

}

#undef LOCTEXT_NAMESPACE
