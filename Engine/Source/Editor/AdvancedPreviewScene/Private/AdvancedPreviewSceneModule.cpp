// Copyright Epic Games, Inc. All Rights Reserved.

#include "AdvancedPreviewSceneModule.h"

#include "AdvancedPreviewSceneCommands.h"
#include "Modules/ModuleManager.h"
#include "SAdvancedPreviewDetailsTab.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class UObject;

void FAdvancedPreviewSceneModule::StartupModule()
{
	FAdvancedPreviewSceneCommands::Register();
}

void FAdvancedPreviewSceneModule::ShutdownModule()
{
}

TSharedRef<SWidget> FAdvancedPreviewSceneModule::CreateAdvancedPreviewSceneSettingsWidget(const TSharedRef<FAdvancedPreviewScene>& InPreviewScene, UObject* InAdditionalSettings, const TArray<FDetailCustomizationInfo>& InDetailCustomizations, const TArray<FPropertyTypeCustomizationInfo>& InPropertyTypeCustomizations, const TArray<FDetailDelegates>& InDelegates)
{
	return SNew(SAdvancedPreviewDetailsTab, InPreviewScene)
		.AdditionalSettings(InAdditionalSettings)
		.DetailCustomizations(InDetailCustomizations)
		.PropertyTypeCustomizations(InPropertyTypeCustomizations)
		.Delegates(InDelegates);

}

IMPLEMENT_MODULE(FAdvancedPreviewSceneModule, AdvancedPreviewScene);
