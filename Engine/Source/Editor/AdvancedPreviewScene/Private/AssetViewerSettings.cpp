// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetViewerSettings.h"
#include "UObject/UnrealType.h"
#include "Editor/EditorPerProjectUserSettings.h"
#include "Editor.h"
#include "Materials/MaterialInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetViewerSettings)

FName UDefaultEditorProfiles::DefaultProfileName = FName("Epic Headquarters");
FName UDefaultEditorProfiles::EditingProfileName = FName("Grey Wireframe");
FName UDefaultEditorProfiles::GreyAmbientProfileName = FName("Grey Ambient");

EProfileConfigVersion USharedProfiles::GetVersion() const
{
	return Version;
}

void USharedProfiles::SetVersionToLatest()
{
	Version = EProfileConfigVersion::LatestVersion;
}

UStaticMesh* FPreviewSceneProfile::GetEnvironmentFloorMesh() const
{
	if (!EnvironmentFloorMesh && !EnvironmentFloorMeshPath.IsEmpty())
	{
		FPreviewSceneProfile* MutableThis = const_cast<FPreviewSceneProfile*>(this);
		MutableThis->EnvironmentFloorMesh = LoadObject<UStaticMesh>(nullptr, *EnvironmentFloorMeshPath);
	}

	return EnvironmentFloorMesh.Get();
}

UMaterialInterface* FPreviewSceneProfile::GetEnvironmentFloorMaterial() const
{
	if (!EnvironmentFloorMaterial && !EnvironmentFloorMaterialPath.IsEmpty())
	{
		FPreviewSceneProfile* MutableThis = const_cast<FPreviewSceneProfile*>(this);
		MutableThis->EnvironmentFloorMaterial = LoadObject<UMaterialInterface>(nullptr, *EnvironmentFloorMaterialPath);
	}

	return EnvironmentFloorMaterial.Get();
}

void FPreviewSceneProfile::LoadProfileObjects()
{
	LoadEnvironmentMap();
	GetEnvironmentFloorMaterial();
	GetEnvironmentFloorMesh();
}

void FPreviewSceneProfile::PostEditChangeProperty(FPropertyChangedEvent& InEvent)
{
	// Store path to the set environment map texture
	EnvironmentCubeMapPath = EnvironmentCubeMap.ToString();
	EnvironmentFloorMeshPath = EnvironmentFloorMesh.ToString();
	EnvironmentFloorMaterialPath = EnvironmentFloorMaterial.ToString();
}

const FPreviewSceneProfile* UDefaultEditorProfiles::GetProfile(const FString& ProfileName)
{
	for (const FPreviewSceneProfile& Profile : Profiles)
	{
		if (Profile.ProfileName == ProfileName)
		{
			return &Profile;
		}
	}

	return nullptr;
}

UAssetViewerSettings::UAssetViewerSettings()
{
}

UAssetViewerSettings::~UAssetViewerSettings()
{
	if (GEditor)
	{
		GEditor->UnregisterForUndo(this);
	}
}

UAssetViewerSettings* UAssetViewerSettings::Get()
{
	// This is a singleton, use default object
	UAssetViewerSettings* DefaultSettings = GetMutableDefault<UAssetViewerSettings>();

	// Load environment map textures (once)
	static bool bInitialized = false;
	if (!bInitialized)
	{
		DefaultSettings->SetFlags(RF_Transactional);

		for (FPreviewSceneProfile& Profile : DefaultSettings->Profiles)
		{
			Profile.LoadProfileObjects();
		}
		bInitialized = true;

		if (GEditor)
		{
			GEditor->RegisterForUndo(DefaultSettings);
		}
	}

	return DefaultSettings;
}

FPreviewSceneProfile& UAssetViewerSettings::GetCurrentUserProjectProfile()
{
	return Get()->Profiles[GetMutableDefault<UEditorPerProjectUserSettings>()->AssetViewerProfileIndex];
}

void UAssetViewerSettings::Save(bool bWarnIfFail)
{
	ULocalProfiles* LocalProfilesObject = GetMutableDefault<ULocalProfiles>();
	USharedProfiles* SharedProfilesObject = GetMutableDefault<USharedProfiles>();

	TArray<FPreviewSceneProfile>& LocalProfiles = GetMutableDefault<ULocalProfiles>()->Profiles;
	TArray<FPreviewSceneProfile>& SharedProfiles = GetMutableDefault<USharedProfiles>()->Profiles;

	LocalProfiles.Empty();
	SharedProfiles.Empty();

	// Divide profiles up in corresponding arrays
	for (FPreviewSceneProfile& Profile : Profiles)
	{
		if (Profile.bSharedProfile)
		{
			SharedProfiles.Add(Profile);
		}
		else
		{
			LocalProfiles.Add(Profile);
		}
	}

	LocalProfilesObject->SaveConfig();

	SharedProfilesObject->SaveConfig();
	SharedProfilesObject->TryUpdateDefaultConfigFile(FString(), bWarnIfFail);
}

void UAssetViewerSettings::PostEditChangeProperty(struct FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);
	
	FName PropertyName = InPropertyChangedEvent.GetPropertyName();

	const int32 ProfileIndex = GetMutableDefault<UEditorPerProjectUserSettings>()->AssetViewerProfileIndex;
	if (Profiles.IsValidIndex(ProfileIndex))
	{
		Profiles[ProfileIndex].PostEditChangeProperty(InPropertyChangedEvent);
	}

	if (NumProfiles != Profiles.Num())
	{
		OnAssetViewerProfileAddRemovedEvent.Broadcast();
		NumProfiles = Profiles.Num();
	}

	OnAssetViewerSettingsChangedEvent.Broadcast(PropertyName);
}

void UAssetViewerSettings::PostEditChangeChainProperty(FPropertyChangedChainEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeChainProperty(InPropertyChangedEvent);

	FName PropertyName = NAME_None;

	// Get the inner property of Profiles
	if (FEditPropertyChain::TDoubleLinkedListNode* ProfileNode = InPropertyChangedEvent.PropertyChain.GetActiveMemberNode())
	{
		ProfileNode = ProfileNode->GetNextNode();
		if (ProfileNode)
		{
			PropertyName = ProfileNode->GetValue()->GetFName();
		}
	}

	const int32 ProfileIndex = GetMutableDefault<UEditorPerProjectUserSettings>()->AssetViewerProfileIndex;
	if (Profiles.IsValidIndex(ProfileIndex))
	{
		Profiles[ProfileIndex].PostEditChangeProperty(InPropertyChangedEvent);
	}

	if (NumProfiles != Profiles.Num())
	{
		OnAssetViewerProfileAddRemovedEvent.Broadcast();
		NumProfiles = Profiles.Num();
	}
	
	OnAssetViewerSettingsChangedEvent.Broadcast(PropertyName);
}

void UAssetViewerSettings::PostInitProperties()
{
	Super::PostInitProperties();

	Profiles.Empty();
	
	USharedProfiles* SharedProfilesDefault = GetMutableDefault<USharedProfiles>();
	TArray<FPreviewSceneProfile>& SharedProfiles = SharedProfilesDefault->Profiles;
	for (FPreviewSceneProfile& Profile : SharedProfiles)
	{
		// Handle migration here, this is done after config is loaded to update outdated properties
		if (SharedProfilesDefault->GetVersion() < EProfileConfigVersion::ApplyNewGridMaterial)
		{
			// if it's an engine profile and material was not overriden
			if (Profile.EnvironmentFloorMaterialPath.IsEmpty() && Profile.bIsEngineDefaultProfile)
			{
				// Apply new default floor material
				Profile.EnvironmentFloorMaterialPath = TEXT("/Engine/EngineMaterials/M_Grid.M_Grid");
			}
		}

		Profiles.Add(Profile);
	}
	// Set the new version to avoid triggering migration again
	SharedProfilesDefault->SetVersionToLatest();

	TArray<FPreviewSceneProfile>& LocalProfiles = GetMutableDefault<ULocalProfiles>()->Profiles;
	for (FPreviewSceneProfile& Profile : LocalProfiles)
	{
		Profiles.Add(Profile);
	}
	
	TArray<FPreviewSceneProfile>& DefaultEditorProfiles = GetMutableDefault<UDefaultEditorProfiles>()->Profiles;
	for (FPreviewSceneProfile& ProfileToAdd : DefaultEditorProfiles)
	{
		// add the default profile if it's not already stored
		// default editor profiles should be marked as "shared" profiles to allow user overrides
		// therefore, they are only added once and maintained by the user thereafter
		if (!Profiles.ContainsByPredicate([&ProfileToAdd](const FPreviewSceneProfile& Profile)
			{
				return Profile.ProfileName == ProfileToAdd.ProfileName;
			}))
		{
			Profiles.Add(ProfileToAdd);
		}
	}

	// the UDefaultEditorProfiles should always add at least one default profile
	if (!ensure(!Profiles.IsEmpty()))
	{
		// Make sure there always is one profile as default
		Profiles.AddDefaulted(1);
		Profiles[0].ProfileName = TEXT("Profile_0");
	}

	NumProfiles = Profiles.Num();

	UEditorPerProjectUserSettings* ProjectSettings = GetMutableDefault<UEditorPerProjectUserSettings>();
	if (!Profiles.IsValidIndex(ProjectSettings->AssetViewerProfileIndex))
	{
		ProjectSettings->AssetViewerProfileIndex = 0;
	}
}

void UAssetViewerSettings::PostUndo(bool bSuccess)
{
	if (bSuccess)
	{
		OnAssetViewerSettingsPostUndoEvent.Broadcast();
	}
}

void UAssetViewerSettings::PostRedo(bool bSuccess)
{
	PostUndo(bSuccess);
}
