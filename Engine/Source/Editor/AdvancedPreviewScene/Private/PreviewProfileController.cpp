// Copyright Epic Games, Inc. All Rights Reserved.

#include "PreviewProfileController.h"

#include "Editor/EditorPerProjectUserSettings.h"
#include "AssetViewerSettings.h"

FPreviewProfileController::FPreviewProfileController()
{
	PerProjectSettings = GetMutableDefault<UEditorPerProjectUserSettings>();
	if (PerProjectSettings)
	{
		AssetViewerSettings = UAssetViewerSettings::Get();
		if (AssetViewerSettings)
		{
			AssetViewerSettingsProfileAddRemoveHandle = AssetViewerSettings->OnAssetViewerProfileAddRemoved().AddLambda([this]()
			{
				UpdateAssetViewerProfiles();
				OnPreviewProfileListChanged().Broadcast();
			});

			AssetViewerSettingsChangedHandle = AssetViewerSettings->OnAssetViewerSettingsChanged().AddLambda([this](const FName& InPropertyName)
			{
				FString CurrProfileName = AssetViewerProfileNames[CurrentProfileIndex];
				UpdateAssetViewerProfiles();
				if (CurrProfileName != AssetViewerProfileNames[CurrentProfileIndex])
				{
					OnPreviewProfileChanged().Broadcast();
				}
			});
			
			UpdateAssetViewerProfiles();
		}
	}
}

FPreviewProfileController::~FPreviewProfileController()
{
	if (IsValid(AssetViewerSettings))
	{
		AssetViewerSettings->OnAssetViewerProfileAddRemoved().Remove(AssetViewerSettingsProfileAddRemoveHandle);
		AssetViewerSettings->OnAssetViewerSettingsChanged().Remove(AssetViewerSettingsChangedHandle);
	}
}

void FPreviewProfileController::UpdateAssetViewerProfiles()
{
	AssetViewerProfileNames.Empty();

	if (AssetViewerSettings && PerProjectSettings)
	{
		// Rebuild the profile list.
		for (const FPreviewSceneProfile& Profile : AssetViewerSettings->Profiles)
		{
			AssetViewerProfileNames.Add(Profile.ProfileName);
		}

		CurrentProfileIndex = PerProjectSettings->AssetViewerProfileIndex;

		EnsureProfilesStateCoherence();
	}
}

TArray<FString> FPreviewProfileController::GetPreviewProfiles(int32& OutCurrentProfileIndex) const
{
	if (PerProjectSettings && AssetViewerSettings)
	{
		EnsureProfilesStateCoherence();
		OutCurrentProfileIndex = CurrentProfileIndex;
	}
	return AssetViewerProfileNames;
}

bool FPreviewProfileController::SetActiveProfile(const FString& ProfileName)
{
	if (IsValid(PerProjectSettings) && IsValid(AssetViewerSettings))
	{
		EnsureProfilesStateCoherence();

		int32 Index = AssetViewerProfileNames.IndexOfByKey(ProfileName);
		if (Index != INDEX_NONE && Index != PerProjectSettings->AssetViewerProfileIndex)
		{
			// Store the settings.
			PerProjectSettings->AssetViewerProfileIndex = Index;
			CurrentProfileIndex = Index;

			// Notify the observer about the change.
			AssetViewerSettings->OnAssetViewerSettingsChanged().Broadcast(GET_MEMBER_NAME_CHECKED(FPreviewSceneProfile, ProfileName));
			return true;
		}
	}
	return false;
}

FString FPreviewProfileController::GetActiveProfile() const
{
	if (PerProjectSettings && AssetViewerSettings)
	{
		EnsureProfilesStateCoherence();
		return AssetViewerProfileNames[PerProjectSettings->AssetViewerProfileIndex];
	}
	return FString();
}

bool FPreviewProfileController::HasAnyUserProfiles() const
{
	if (AssetViewerSettings)
	{
		for (const FPreviewSceneProfile& Profile : AssetViewerSettings->Profiles)
		{
			if (!Profile.bIsEngineDefaultProfile)
			{
				return true;
			}
		}
	}
	
	return false;
}

void FPreviewProfileController::EnsureProfilesStateCoherence() const
{
	ensureMsgf(AssetViewerProfileNames.Num() == AssetViewerSettings->Profiles.Num(), TEXT("List of profiles is out of sync with the list of corresponding profile names."));
	ensureMsgf(AssetViewerProfileNames.Num() > 0, TEXT("The list of profiles is expected to always have at least one default profile"));
	ensureMsgf(AssetViewerSettings->Profiles.IsValidIndex(PerProjectSettings->AssetViewerProfileIndex), TEXT("The active asset viewer profile is invalid."));
	ensureMsgf(CurrentProfileIndex == PerProjectSettings->AssetViewerProfileIndex, TEXT("The cached selected profile index is out of sync."));
}
