// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAdvancedPreviewDetailsTab.h"
#include "Editor/EditorPerProjectUserSettings.h"
#include "Modules/ModuleManager.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/STextComboBox.h"
#include "IDetailsView.h"
#include "PropertyEditorModule.h"
#include "ScopedTransaction.h"

#include "AssetViewerSettings.h"

#include "AdvancedPreviewScene.h"
#include "IDetailRootObjectCustomization.h"
#include "Misc/MessageDialog.h"

#define LOCTEXT_NAMESPACE "SPrettyPreview"

SAdvancedPreviewDetailsTab::SAdvancedPreviewDetailsTab()
{
	DefaultSettings = UAssetViewerSettings::Get();
	if (DefaultSettings)
	{
		RefreshDelegate = DefaultSettings->OnAssetViewerSettingsChanged().AddRaw(this, &SAdvancedPreviewDetailsTab::OnAssetViewerSettingsRefresh);
			
		AddRemoveProfileDelegate = DefaultSettings->OnAssetViewerProfileAddRemoved().AddLambda([this]() { this->Refresh(); });

		PostUndoDelegate = DefaultSettings->OnAssetViewerSettingsPostUndo().AddRaw(this, &SAdvancedPreviewDetailsTab::OnAssetViewerSettingsPostUndo);
	}

	PerProjectSettings = GetMutableDefault<UEditorPerProjectUserSettings>();

}

SAdvancedPreviewDetailsTab::~SAdvancedPreviewDetailsTab()
{
	DefaultSettings = UAssetViewerSettings::Get();
	if (DefaultSettings)
	{
		DefaultSettings->OnAssetViewerSettingsChanged().Remove(RefreshDelegate);
		DefaultSettings->OnAssetViewerProfileAddRemoved().Remove(AddRemoveProfileDelegate);
		DefaultSettings->OnAssetViewerSettingsPostUndo().Remove(PostUndoDelegate);
		DefaultSettings->Save();
	}
}

void SAdvancedPreviewDetailsTab::Construct(const FArguments& InArgs, const TSharedRef<FAdvancedPreviewScene>& InPreviewScene)
{
	PreviewScenePtr = InPreviewScene;
	DefaultSettings = UAssetViewerSettings::Get();
	AdditionalSettings = InArgs._AdditionalSettings;
	ProfileIndex = DefaultSettings->Profiles.IsValidIndex(PerProjectSettings->AssetViewerProfileIndex) ? PerProjectSettings->AssetViewerProfileIndex : 0;
	DetailCustomizations = InArgs._DetailCustomizations;
	PropertyTypeCustomizations = InArgs._PropertyTypeCustomizations;
	Delegates = InArgs._Delegates;

	CreateSettingsView();

	this->ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()		
		.Padding(2.0f, 1.0f, 2.0f, 1.0f)
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.Padding(2.0f)
			[
				SAssignNew(ProfileComboBox, STextComboBox)
				.ToolTipText(LOCTEXT("SceneProfileComboBoxToolTip", "Allows for switching between scene environment and lighting profiles."))
				.OptionsSource(&ProfileNames)
				.OnSelectionChanged(this, &SAdvancedPreviewDetailsTab::ComboBoxSelectionChanged)				
				.IsEnabled_Lambda([this]() -> bool
				{
					return !ProfileNames.IsEmpty();
				})
			]
			+ SHorizontalBox::Slot()
			.Padding(2.0f)
			.AutoWidth()
			[
				SNew(SButton)
				.OnClicked(this, &SAdvancedPreviewDetailsTab::AddProfileButtonClick)
				.Text(LOCTEXT("AddProfileButton", "Add Profile"))
				.ToolTipText(LOCTEXT("SceneProfileAddProfile", "Adds a new profile."))
			]
			+ SHorizontalBox::Slot()
			.Padding(2.0f)
			.AutoWidth()
			[
				SNew(SButton)
				.OnClicked(this, &SAdvancedPreviewDetailsTab::CloneProfileButtonClick)
				.Text(LOCTEXT("CloneProfileButton", "Clone Profile"))
				.ToolTipText(LOCTEXT("SceneProfileCloneProfile", "Clones this profile into a new profile."))
			]
			+ SHorizontalBox::Slot()
			.Padding(2.0f)
			.AutoWidth()
			[
				SNew(SButton)
				.OnClicked(this, &SAdvancedPreviewDetailsTab::RemoveOrResetProfileButtonClick)
				.Text_Lambda([this]()
				{
					if (DefaultSettings->Profiles[ProfileIndex].bIsEngineDefaultProfile)
					{
						return LOCTEXT("ResetProfileButton", "Reset Profile");
					}
					return LOCTEXT("RemoveProfileButton", "Remove Profile");
				})
				.ToolTipText_Lambda([this]()
				{
					if (DefaultSettings->Profiles[ProfileIndex].bIsEngineDefaultProfile)
					{
						return LOCTEXT("SceneProfileResetProfile", "Resets this engine profile to default settings. Cannot delete engine profiles.");
					}
					return LOCTEXT("SceneProfileRemoveProfile", "Removes the currently selected profile.");
				})
				.IsEnabled_Lambda([this]()->bool
				{
					return ProfileNames.Num() > 1; 
				})
			]
		]
		+ SVerticalBox::Slot()
		.Padding(2.0f, 1.0f, 2.0f, 1.0f)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			[
				SettingsView->AsShared()
			]
		]
	];

	UpdateProfileNames();
	UpdateSettingsView();
}

void SAdvancedPreviewDetailsTab::ComboBoxSelectionChanged(TSharedPtr<FString> NewSelection, ESelectInfo::Type SelectInfo)
{
	int32 NewSelectionIndex;

	if (ProfileNames.Find(NewSelection, NewSelectionIndex))
	{
		if (const TSharedPtr<FAdvancedPreviewScene> PreviewScene = PreviewScenePtr.Pin())
		{
			// No need to refresh anything
			if (NewSelectionIndex == ProfileIndex 
				&& ProfileIndex == PerProjectSettings->AssetViewerProfileIndex
				&& PreviewScene->GetCurrentProfileIndex() == ProfileIndex)
			{
				return;
			}

			ProfileIndex = NewSelectionIndex;
			PerProjectSettings->AssetViewerProfileIndex = ProfileIndex;
			UpdateSettingsView();	
			if (SelectInfo == ESelectInfo::Type::OnMouseClick)
			{
				PreviewScene->SetProfileIndex(ProfileIndex);	
			}
		}
	}
}

void SAdvancedPreviewDetailsTab::UpdateSettingsView()
{	
	TArray<UObject*> Objects;
	if (AdditionalSettings)
	{
		Objects.Add(AdditionalSettings);
	}
	Objects.Add(DefaultSettings);

	SettingsView->SetObjects(Objects, true);
}

void SAdvancedPreviewDetailsTab::UpdateProfileNames()
{
	checkf(DefaultSettings->Profiles.Num(), TEXT("There should always be at least one profile available"));
	ProfileNames.Empty();
	for (FPreviewSceneProfile& Profile : DefaultSettings->Profiles)
	{
		FString Suffix = TEXT("");
		if (Profile.bSharedProfile)
		{
			Suffix = TEXT(" (Shared)");
		}
		if (Profile.bIsEngineDefaultProfile)
		{
			Suffix = TEXT(" (Engine Default)");
		}
		ProfileNames.Add(TSharedPtr<FString>(new FString(Profile.ProfileName + Suffix)));
	}

	ProfileComboBox->RefreshOptions();
	ProfileComboBox->SetSelectedItem(ProfileNames[ProfileIndex]);
}

FReply SAdvancedPreviewDetailsTab::AddProfileButtonClick()
{
	const FScopedTransaction Transaction(LOCTEXT("AddSceneProfile", "Adding Preview Scene Profile"));
	DefaultSettings->Modify();

	// Add new profile to settings instance
	ProfileIndex = DefaultSettings->Profiles.AddDefaulted();
	FPreviewSceneProfile& NewProfile = DefaultSettings->Profiles.Last();
	NewProfile.LoadProfileObjects();

	// Try to create a valid profile name when one is added
	bool bFoundValidName = false;
	int32 ProfileAppendNum = FMath::Max(0, DefaultSettings->Profiles.Num() - 1);
	FString NewProfileName;
	while (!bFoundValidName)
	{
		NewProfileName = FString::Printf(TEXT("Profile_%i"), ProfileAppendNum);

		bool bValidName = true;
		for (const FPreviewSceneProfile& Profile : DefaultSettings->Profiles)
		{
			if (Profile.ProfileName == NewProfileName)
			{
				bValidName = false;				
				break;
			}
		}

		if (!bValidName)
		{
			++ProfileAppendNum;
		}

		bFoundValidName = bValidName;
	}

	NewProfile.ProfileName = NewProfileName;
	PerProjectSettings->AssetViewerProfileIndex = ProfileIndex;
	DefaultSettings->PostEditChange();

	// Change selection to new profile so the user directly sees the profile that was added
	Refresh();
	ProfileComboBox->SetSelectedItem(ProfileNames.Last());
	
	return FReply::Handled();
}

FReply SAdvancedPreviewDetailsTab::CloneProfileButtonClick()
{
	const FScopedTransaction Transaction(LOCTEXT("CloneSceneProfile", "Cloning Preview Scene Profile"));
    DefaultSettings->Modify();

    // Add new profile to settings instance based on current profile
	const int32 CurrentProfileIndex = DefaultSettings->Profiles.IsValidIndex(ProfileIndex) ? ProfileIndex : 0;
	FPreviewSceneProfile CurrentProfile = DefaultSettings->Profiles[CurrentProfileIndex];
	CurrentProfile.bIsEngineDefaultProfile = false;
	CurrentProfile.bSharedProfile = false;
    ProfileIndex = DefaultSettings->Profiles.Add(CurrentProfile);
    FPreviewSceneProfile& NewProfile = DefaultSettings->Profiles.Last();

    // Try to create a valid profile name when one is added
    bool bFoundValidName = false;
    int32 ProfileAppendNum = FMath::Max(0, DefaultSettings->Profiles.Num() - 1);
    FString NewProfileName;
    while (!bFoundValidName)
    {
    	NewProfileName = FString::Printf(TEXT("Profile_%i"), ProfileAppendNum);
    
    	bool bValidName = true;
    	for (const FPreviewSceneProfile& Profile : DefaultSettings->Profiles)
    	{
    		if (Profile.ProfileName == NewProfileName)
    		{
    			bValidName = false;				
    			break;
    		}
    	}
    
    	if (!bValidName)
    	{
    		++ProfileAppendNum;
    	}
    
    	bFoundValidName = bValidName;
    }

    NewProfile.ProfileName = NewProfileName;
    PerProjectSettings->AssetViewerProfileIndex = ProfileIndex;
    DefaultSettings->PostEditChange();

    // Change selection to new profile so the user directly sees the profile that was added
    Refresh();
    ProfileComboBox->SetSelectedItem(ProfileNames.Last());

    return FReply::Handled();
}

FReply SAdvancedPreviewDetailsTab::RemoveOrResetProfileButtonClick()
{
	const FPreviewSceneProfile& CurrentProfile = DefaultSettings->Profiles[ProfileIndex];

	const EAppReturnType::Type Result = FMessageDialog::Open(
		EAppMsgType::YesNo,
		CurrentProfile.bIsEngineDefaultProfile 
			? FText::Format(LOCTEXT("ResetSceneProfileDialogMessage", "Do you want to reset this profile ?\n{0}"), FText::FromString(CurrentProfile.ProfileName))
			: FText::Format(LOCTEXT("RemoveSceneProfileDialogMessage", "Do you want to remove this profile ?\n{0}"), FText::FromString(CurrentProfile.ProfileName))
	);

	if (Result == EAppReturnType::No || Result == EAppReturnType::Cancel)
	{
		return FReply::Handled();
	}

	const FScopedTransaction Transaction(
		CurrentProfile.bIsEngineDefaultProfile 
			? FText::Format(LOCTEXT("ResetSceneProfile", "Reset Preview Scene Profile ({0})"), FText::FromString(CurrentProfile.ProfileName))
			: FText::Format(LOCTEXT("RemoveSceneProfile", "Remove Preview Scene Profile ({0})"), FText::FromString(CurrentProfile.ProfileName))
	);
	DefaultSettings->Modify();

	if (CurrentProfile.bIsEngineDefaultProfile)
	{
		if (const FPreviewSceneProfile* DefaultEditorProfile = GetMutableDefault<UDefaultEditorProfiles>()->GetProfile(CurrentProfile.ProfileName))
		{
			// Reset currently selected profile 
			DefaultSettings->Profiles[ProfileIndex] = *DefaultEditorProfile;
			DefaultSettings->Profiles[ProfileIndex].LoadProfileObjects();
			DefaultSettings->PostEditChange();
			return FReply::Handled();
		}

		// if we get here, it means an engine-provided default profile was removed from the engine,
		// in which case we should remove it...
	}

	// Remove currently selected profile 
	DefaultSettings->Profiles.RemoveAt(ProfileIndex);
	ProfileIndex = DefaultSettings->Profiles.IsValidIndex(ProfileIndex - 1 ) ? ProfileIndex - 1 : 0;
	PerProjectSettings->AssetViewerProfileIndex = ProfileIndex;
	DefaultSettings->PostEditChange();
	return FReply::Handled();
}

void SAdvancedPreviewDetailsTab::OnAssetViewerSettingsRefresh(const FName& InPropertyName)
{
	if (!PreviewScenePtr.IsValid())
	{
		// this callback can fire when the editor is forcibly closed and tool modes revert the active profile
		// when this happens, the preview scene is null even though this details tab hasn't been destroyed yet (and unregistered this delegate)
		return;
	}
	
	if (InPropertyName == GET_MEMBER_NAME_CHECKED(FPreviewSceneProfile, ProfileName) 
		|| InPropertyName == GET_MEMBER_NAME_CHECKED(FPreviewSceneProfile, bSharedProfile)
		|| ProfileIndex != PreviewScenePtr.Pin()->GetCurrentProfileIndex())
	{
		Refresh();
	}
}

void SAdvancedPreviewDetailsTab::CreateSettingsView()
{	
	// Create a property view
	FPropertyEditorModule& EditModule = FModuleManager::Get().GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.bHideSelectionTip = true;
	DetailsViewArgs.DefaultsOnlyVisibility = EEditDefaultsOnlyNodeVisibility::Automatic;
	DetailsViewArgs.bShowOptions = false;
	DetailsViewArgs.bAllowMultipleTopLevelObjects = true;

	SettingsView = EditModule.CreateDetailView(DetailsViewArgs);

	for (const FAdvancedPreviewSceneModule::FDetailCustomizationInfo& DetailCustomizationInfo : DetailCustomizations)
	{
		SettingsView->RegisterInstancedCustomPropertyLayout(DetailCustomizationInfo.Struct, DetailCustomizationInfo.OnGetDetailCustomizationInstance);
	}

	for (const FAdvancedPreviewSceneModule::FPropertyTypeCustomizationInfo& PropertyTypeCustomizationInfo : PropertyTypeCustomizations)
	{
		SettingsView->RegisterInstancedCustomPropertyTypeLayout(PropertyTypeCustomizationInfo.StructName, PropertyTypeCustomizationInfo.OnGetPropertyTypeCustomizationInstance);
	}

	for (FAdvancedPreviewSceneModule::FDetailDelegates& DetailDelegate : Delegates)
	{
		DetailDelegate.OnPreviewSceneChangedDelegate.AddSP(this, &SAdvancedPreviewDetailsTab::OnPreviewSceneChanged);
	}

	UpdateSettingsView();
}

void SAdvancedPreviewDetailsTab::Refresh()
{	
	PerProjectSettings->AssetViewerProfileIndex = DefaultSettings->Profiles.IsValidIndex(PerProjectSettings->AssetViewerProfileIndex) ? PerProjectSettings->AssetViewerProfileIndex : 0;
	ProfileIndex = PerProjectSettings->AssetViewerProfileIndex;
	UpdateProfileNames();
	PreviewScenePtr.Pin()->SetProfileIndex(ProfileIndex);
	UpdateSettingsView();
}

void SAdvancedPreviewDetailsTab::OnAssetViewerSettingsPostUndo()
{
	Refresh();
	PreviewScenePtr.Pin()->UpdateScene(DefaultSettings->Profiles[ProfileIndex]);
}

void SAdvancedPreviewDetailsTab::OnPreviewSceneChanged(TSharedRef<FAdvancedPreviewScene> PreviewScene)
{
  	PreviewScenePtr = PreviewScene;
	Refresh();
}

#undef LOCTEXT_NAMESPACE