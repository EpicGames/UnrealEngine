// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewportInteractions/ViewportLightRotationInteraction.h"

#include "AdvancedPreviewScene.h"
#include "AssetViewerSettings.h"
#include "Components/DirectionalLightComponent.h"
#include "EditorViewportClient.h"
#include "PrimitiveDrawingUtils.h"
#include "Behaviors/ViewportClickDragBehavior.h"
#include "SceneView.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "ViewportInteractions/ViewportInteractionsBehaviorSource.h"

UViewportLightRotationInteraction::UViewportLightRotationInteraction()
{
	using namespace UE::Editor::ViewportInteractions;

	InteractionName = TEXT("LightRotation");

	ViewportClickDragBehavior->SetBindings({
		FButtonBinding(EKeys::LeftMouseButton).TriggersStart().RequiredToStart(false).RequiredToContinue(false),
		FButtonBinding(EKeys::RightMouseButton).TriggersStart().RequiredToStart(false).RequiredToContinue(false),
		FButtonBinding(EKeys::MiddleMouseButton).TriggersStart().RequiredToStart(false).RequiredToContinue(false),
		FButtonBinding(EKeys::L)
	});
}

void UViewportLightRotationInteraction::Render(IToolsContextRenderAPI* InRenderAPI)
{
	if (!IsActive())
	{
		return;
	}

	const FAdvancedPreviewScene* PreviewScene = GetAdvancedPreviewScene();
	if (!PreviewScene || !PreviewScene->DirectionalLight)
	{
		return;
	}

	FPrimitiveDrawInterface* PDI = InRenderAPI->GetPrimitiveDrawInterface();
	const FSceneView* View = InRenderAPI->GetSceneView();
	if (!PDI || !View)
	{
		return;
	}

	const ULightComponent* Light = PreviewScene->DirectionalLight;

	const FLinearColor ArrowColor = Light->LightColor;

	FTransform LightLocalToWorld = Light->GetComponentToWorld();
	if (Light->IsA<UDirectionalLightComponent>())
	{
		LightLocalToWorld.SetTranslation(FVector::ZeroVector);
	}

	LightLocalToWorld.SetScale3D(FVector(1.0f));

	// Deproject the current mouse position into world space
	FVector MouseWorldPos, MouseWorldDir;
	View->DeprojectFVector2D(ScreenPosition, MouseWorldPos, MouseWorldDir);

	// Find the closest point to the origin along the ray
	MouseWorldPos = FMath::ClosestPointOnLine(MouseWorldPos, MouseWorldPos + MouseWorldDir * WORLD_MAX, FVector::ZeroVector);

	// Compute arrow geometry
	static constexpr float MinMouseRadius = 100.0f;
	static constexpr float MinArrowLength = 10.0f;
	static constexpr float ArrowLengthToSizeRatio = 0.1f;
	static constexpr float MouseLengthToArrowLengthRatio = 0.2f;
	static constexpr float ArrowLengthToThicknessRatio = 0.05f;
	static constexpr float MinArrowThickness = 2.0f;

	const FVector LightToMousePos = MouseWorldPos - LightLocalToWorld.GetTranslation();
	const float LightToMouseRadius = FMath::Max<FVector::FReal>(LightToMousePos.Size(), MinMouseRadius);

	const float ArrowLength = FMath::Max(MinArrowLength, LightToMouseRadius * MouseLengthToArrowLengthRatio);
	const float ArrowSize = ArrowLengthToSizeRatio * ArrowLength;
	const float ArrowThickness = FMath::Max(ArrowLengthToThicknessRatio * ArrowLength, MinArrowThickness);

	const FVector ArrowOrigin = LightLocalToWorld.TransformPosition(FVector(-LightToMouseRadius - 0.5f * ArrowLength, 0.0f, 0.0f));
	const FQuatRotationTranslationMatrix ArrowToWorld(LightLocalToWorld.GetRotation(), ArrowOrigin);

	DrawDirectionalArrow(PDI, ArrowToWorld, ArrowColor, ArrowLength, ArrowSize, SDPG_World, ArrowThickness);
}

bool UViewportLightRotationInteraction::CanBeActivated() const
{
	if (Super::CanBeActivated())
	{
		return GetAdvancedPreviewScene() != nullptr;
	}
	return false;
}

void UViewportLightRotationInteraction::OnDrag(const FDragArgs& InDrag)
{
	// Cache screen position for the render
	ScreenPosition = InDrag.Ray.ScreenPosition;
	Super::OnDrag(InDrag);
}

void UViewportLightRotationInteraction::OnDragDelta(float InMouseDeltaX, float InMouseDeltaY)
{
	if (FAdvancedPreviewScene* PreviewScene = GetAdvancedPreviewScene())
	{
		static constexpr float LightRotSpeed = 0.22f;
		FRotator LightDir = PreviewScene->GetLightDirection();
		LightDir.Yaw += -InMouseDeltaX * LightRotSpeed;
		LightDir.Pitch += -InMouseDeltaY * LightRotSpeed;
		PreviewScene->SetLightDirection(LightDir);

		// Persist the rotation to the current profile settings
		if (UAssetViewerSettings* Settings = UAssetViewerSettings::Get())
		{
			const int32 ProfileIndex = PreviewScene->GetCurrentProfileIndex();
			if (Settings->Profiles.IsValidIndex(ProfileIndex))
			{
				Settings->Profiles[ProfileIndex].DirectionalLightRotation = LightDir;
			}
		}
	}
}

FAdvancedPreviewScene* UViewportLightRotationInteraction::GetAdvancedPreviewScene() const
{
	if (FEditorViewportClient* ViewportClient = GetEditorViewportClient())
	{
		return static_cast<FAdvancedPreviewScene*>(ViewportClient->GetPreviewScene());
	}
	return nullptr;
}
