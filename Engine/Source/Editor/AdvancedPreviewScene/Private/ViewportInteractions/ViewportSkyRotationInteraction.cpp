// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewportInteractions/ViewportSkyRotationInteraction.h"

#include "AdvancedPreviewScene.h"
#include "EditorViewportClient.h"
#include "Behaviors/ViewportClickDragBehavior.h"

UViewportSkyRotationInteraction::UViewportSkyRotationInteraction()
{
	using namespace UE::Editor::ViewportInteractions;

	InteractionName = TEXT("SkyRotation");

	ViewportClickDragBehavior->SetBindings({
		FButtonBinding(EKeys::LeftMouseButton).TriggersStart().RequiredToStart(false).RequiredToContinue(false),
		FButtonBinding(EKeys::RightMouseButton).TriggersStart().RequiredToStart(false).RequiredToContinue(false),
		FButtonBinding(EKeys::MiddleMouseButton).TriggersStart().RequiredToStart(false).RequiredToContinue(false),
		FButtonBinding(EKeys::K)
	});
}

bool UViewportSkyRotationInteraction::CanBeActivated() const
{
	if (Super::CanBeActivated())
	{
		return GetAdvancedPreviewScene() != nullptr;
	}
	return false;
}

void UViewportSkyRotationInteraction::OnDragDelta(float InMouseDeltaX, float InMouseDeltaY)
{
	if (FAdvancedPreviewScene* PreviewScene = GetAdvancedPreviewScene())
	{
		static constexpr float SkyRotationSpeed = 0.22f;
		float SkyRotation = PreviewScene->GetSkyRotation();
		SkyRotation += -InMouseDeltaX * SkyRotationSpeed;
		PreviewScene->SetSkyRotation(SkyRotation);
	}
}

FAdvancedPreviewScene* UViewportSkyRotationInteraction::GetAdvancedPreviewScene() const
{
	if (FEditorViewportClient* ViewportClient = GetEditorViewportClient())
	{
		return static_cast<FAdvancedPreviewScene*>(ViewportClient->GetPreviewScene());
	}
	return nullptr;
}
