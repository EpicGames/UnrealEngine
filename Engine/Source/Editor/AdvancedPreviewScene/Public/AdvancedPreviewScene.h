// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PrettyPreviewScene.h: Pretty preview scene definitions.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "AssetViewerSettings.h"
#include "UnrealClient.h"
#include "Stats/Stats.h"
#include "InputCoreTypes.h"
#include "PreviewScene.h"
#include "TickableEditorObject.h"

#define UE_API ADVANCEDPREVIEWSCENE_API

class FEditorViewportClient;
class FViewport;
class UAssetViewerSettings;
class UMaterialInstanceConstant;
class UPostProcessComponent;
class USkyLightComponent;
class UStaticMeshComponent;
class USphereReflectionCaptureComponent;
struct FPreviewSceneProfile;
class FUICommandList;

class FAdvancedPreviewScene : public FPreviewScene, public FTickableEditorObject
{
public:
	/** An event for when a profile changes. Includes the changed profile and (optionally) the name of the property in the profile that changed. */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnProfileChanged, const FPreviewSceneProfile&, FName);

	UE_API FAdvancedPreviewScene(ConstructionValues CVS, float InFloorOffset = 0.0f);
	UE_API ~FAdvancedPreviewScene();

	UE_API void UpdateScene(FPreviewSceneProfile& Profile, bool bUpdateSkyLight = true, bool bUpdateEnvironment = true, bool bUpdatePostProcessing = true, bool bUpdateDirectionalLight = true);

	/** Begin FPreviewScene */
	UE_API virtual FLinearColor GetBackgroundColor() const override;
	/** End FPreviewScene */

	/* Begin FTickableEditorObject */
	UE_API virtual void Tick(float DeltaTime) override;
	virtual ETickableTickType GetTickableTickType() const override { return ETickableTickType::Always; }
	UE_API virtual TStatId GetStatId() const override;
	/* End FTickableEditorObject */
	
	FOnProfileChanged& OnProfileChanged() { return OnProfileChangedDelegate; }

	UE_API const bool HandleViewportInput(FViewport* InViewport, FInputDeviceId DeviceId, FKey Key, float Delta, float DeltaTime, int32 NumSamples, bool bGamepad);
	UE_API const bool HandleInputKey(const FInputKeyEventArgs& EventArgs);

	UE_API void SetSkyRotation(const float SkyRotation);
	/* Sets the visiblity state for the floor/environment by storing it in the scene profile and refreshing the scene, in case bDirect is true it sets the visibility directly and leaves the profile untouched. */
	UE_API bool GetFloorVisibility() const;
	UE_API void SetFloorVisibility(const bool bVisible, const bool bDirect = false);
	UE_API void SetEnvironmentVisibility(const bool bVisible, const bool bDirect = false);
	UE_API float GetFloorOffset() const;
	UE_API void SetFloorOffset(const float InFloorOffset);
	UE_API void SetProfileIndex(const int32 InProfileIndex);
	UE_API FPreviewSceneProfile* GetCurrentProfile() const;

	UE_API const UStaticMeshComponent* GetFloorMeshComponent() const;
	UE_API const float GetSkyRotation() const;
	UE_API const int32 GetCurrentProfileIndex() const;
	UE_API const bool IsUsingPostProcessing() const;

	TSharedPtr<FUICommandList> GetCommandList() const { return UICommandList; }

	/** Toggle the sky sphere on and off */
	UE_API void HandleToggleEnvironment();
	
	/** Whether the sky sphere is on. */
	UE_API bool IsEnvironmentEnabled() const;

	/** Toggle the floor mesh on and off */
	UE_API void HandleToggleFloor();

	/** Whether the floor is on */
	UE_API bool IsFloorEnabled() const;

	/** Toggle the grid on and off. */
	UE_DEPRECATED(5.8, "bShowGrid is deprecated, this function does nothing")
	UE_API void HandleToggleGrid();
	
	/** Whether the grid is enabled. */
	UE_DEPRECATED(5.8, "bShowGrid is deprecated, this function returns false")
	UE_API bool IsGridEnabled() const;

	/** Toggle post processing on and off */
	UE_API void HandleTogglePostProcessing();
	
	/** Whether post processing is enabled */
	UE_API bool IsPostProcessingEnabled() const;
	
	/** Sets the mesh for the floor */
	UE_API void SetFloorMesh(UStaticMesh* InFloorMesh);
	
	/** Sets the floor material on a specific slot, or -1 for all slots */
	UE_API void SetFloorMaterial(UMaterialInterface* InFloorMaterial, int32 InIndex = -1);

	/** Sets the floor mesh rotation */
	UE_API void SetFloorMeshRotation(const FRotator& InRotation);

	/** Sets the floor mesh scale */
	UE_API void SetFloorMeshScale(const FVector& InScale);

	/** Register sky rotation and light rotation viewport interactions on the given behavior source. */
	UE_API static void AddPreviewSceneInteractions(class UViewportInteractionsBehaviorSource* InBehaviorSource);

protected:
	/** Create and map the command list. */
	UE_API virtual void BindCommands();
	
	/** Add commands to a provided command list. */
	UE_API virtual TSharedRef<FUICommandList> CreateCommandList();
	
	/** Handle refreshing the scene when settings change */
	UE_API void OnAssetViewerSettingsRefresh(const FName& InPropertyName);
	
	UE_API bool UsesViewportInteractions(const FEditorViewportClient* InViewportClient) const;

	/** Navigate to the next or previous profile */
	enum class ENavigationDirection : uint8
	{
		Forward,
		Backward,
	};
	UE_API void NavigateProfile(ENavigationDirection InDirection);

protected:
	FOnProfileChanged OnProfileChangedDelegate;

	UStaticMeshComponent* SkyComponent;
	UMaterialInstanceConstant* InstancedSkyMaterial;
	UPostProcessComponent* PostProcessComponent;
	UStaticMeshComponent* FloorMeshComponent;
	UAssetViewerSettings* DefaultSettings;	
	bool bRotateLighting;

	float CurrentRotationSpeed;
	float PreviousRotation;
	float UILightingRigRotationDelta;

	bool bSkyChanged;
	bool bPostProcessing;

	int32 CurrentProfileIndex;

	/** Command list for input handling */
	TSharedPtr<FUICommandList> UICommandList;

	/** Delegate handle used to refresh the scene when settings change */
	FDelegateHandle RefreshDelegate;
};

#undef UE_API
