// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"
#include "Internationalization/Internationalization.h"
#include "Styling/AppStyle.h"
#include "Styling/ISlateStyle.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"

class FUICommandInfo;

PRAGMA_DISABLE_DEPRECATION_WARNINGS // For ToggleGrid
class ADVANCEDPREVIEWSCENE_API FAdvancedPreviewSceneCommands 
	: public TCommands<FAdvancedPreviewSceneCommands>
{

public:
	FAdvancedPreviewSceneCommands() : TCommands<FAdvancedPreviewSceneCommands>
	(
		"AdvancedPreviewScene",
		NSLOCTEXT("Contexts", "AdvancedPreviewScene", "Advanced Preview Scene"),
		NAME_None,
		FAppStyle::Get().GetStyleSetName()
	)
	{}
	
	/** Toggles environment (sky sphere) visibility */
	TSharedPtr< FUICommandInfo > ToggleEnvironment;

	/** Toggles floor visibility */
	TSharedPtr< FUICommandInfo > ToggleFloor;
	
	/** Toggles post processing */
	TSharedPtr< FUICommandInfo > TogglePostProcessing;
	
	/** Toggles the grid */
	UE_DEPRECATED(5.8, "No longer in use, will be removed")
	TSharedPtr< FUICommandInfo > ToggleGrid;
	
	/** Navigate to next profile */
	TSharedPtr<FUICommandInfo> NextProfile;
	
	/** Navigate to previous profile */
	TSharedPtr<FUICommandInfo> PreviousProfile;

	/**
	 * Initialize commands
	 */
	virtual void RegisterCommands() override;
};
PRAGMA_ENABLE_DEPRECATION_WARNINGS // For ToggleGrid
