// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/NameTypes.h"
#include "Templates/SharedPointer.h"

class FUICommandList;
class FAdvancedPreviewScene;
class FEditorViewportClient;
struct FPreviewSceneProfile;

namespace UE::AdvancedPreviewScene::Menus
{

struct FSettingsOptions
{
	bool bShowToggleEnvironment = true;
	UE_DEPRECATED(5.8, "No longer in use, will be removed")
	bool bShowToggleGrid = true;
	bool bShowToggleFloor = true;
	bool bShowTogglePostProcessing = true;

	FSettingsOptions& ShowToggleEnvironment(bool bInShowToggleEnvironment)
	{
		bShowToggleEnvironment = bInShowToggleEnvironment;
		return *this;
	}

	UE_DEPRECATED(5.8, "No longer in use, will be removed")
	FSettingsOptions& ShowToggleGrid(bool bInShowToggleGrid)
	{
		return *this;
	}

	FSettingsOptions& ShowToggleFloor(bool bInShowToggleFloor)
	{
		bShowToggleFloor = bInShowToggleFloor;
		return *this;
	}

	FSettingsOptions& ShowTogglePostProcessing(bool bInShowTogglePostProcessing)
	{
		bShowTogglePostProcessing = bInShowTogglePostProcessing;
		return *this;
	}
};

ADVANCEDPREVIEWSCENE_API void ExtendAdvancedPreviewSceneSettings(FName InAssetViewerProfileSubmenuName, const FSettingsOptions& InSettingsOptions = {});

}

namespace UE::AdvancedPreviewScene
{

/**
 * A default handler for `FAdvancedPreviewScene::OnProfileChanged()`. Syncs the profile to the provided client's show flags and other settings.
 */
ADVANCEDPREVIEWSCENE_API void DefaultOnSettingsChangedHandler(const FPreviewSceneProfile& Profile, FName PropertyName, TWeakPtr<FEditorViewportClient> Client);

/**
 * Attaches `DefaultOnSettingsChangedHandler()` to the provided scene's `OnProfileChanged` delegate & calls the function with the current profile to syncronize the client.
 * This is a convenience that alleviates the annoyance of having to manually downcast subtypes of FEditorViewporClient.
 */
ADVANCEDPREVIEWSCENE_API void BindDefaultOnSettingsChangedHandler(const TSharedPtr<FAdvancedPreviewScene>& Scene, const TSharedPtr<FEditorViewportClient>& Client);

}
