// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "Modules/ModuleInterface.h"
#include "PropertyEditorDelegates.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

class FAdvancedPreviewScene;
class SWidget;
class UObject;
class UStruct;

class FAdvancedPreviewSceneModule : public IModuleInterface
{
public:
	/** Info about a per-instance details customization */
	struct FDetailCustomizationInfo
	{
		UStruct* Struct;
		FOnGetDetailCustomizationInstance OnGetDetailCustomizationInstance;
	};

	/** Info about a per-instance property type customization */
	struct FPropertyTypeCustomizationInfo
	{
		FName StructName;
		FOnGetPropertyTypeCustomizationInstance OnGetPropertyTypeCustomizationInstance;
	};

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnPreviewSceneChanged, TSharedRef<FAdvancedPreviewScene>);

	/** Info about a Delegates to subscribe to */
	struct FDetailDelegates
	{
		FOnPreviewSceneChanged& OnPreviewSceneChangedDelegate;
	};

	// IModuleInterface implementation
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	/**  
	 * Create an advanced preview scene settings widget.
	 * 
	 * @param	InPreviewScene					The preview scene to create the widget for
	 * @param	InAdditionalSettings			Additional settings object to display in the view
	 * @param	InDetailCustomizations			Customizations to use for this details tab
	 * @param	InPropertyTypeCustomizations	Customizations to use for this details tab
	 * @param	InDelegates						Delegates to use for this details tab. Array to match other args
	 * @return a new widget
	 */
	virtual TSharedRef<SWidget> CreateAdvancedPreviewSceneSettingsWidget(const TSharedRef<FAdvancedPreviewScene>& InPreviewScene, UObject* InAdditionalSettings = nullptr, const TArray<FDetailCustomizationInfo>& InDetailCustomizations = TArray<FDetailCustomizationInfo>(), const TArray<FPropertyTypeCustomizationInfo>& InPropertyTypeCustomizations = TArray<FPropertyTypeCustomizationInfo>(), const TArray<FDetailDelegates>& InDelegates = TArray<FDetailDelegates>());
};
