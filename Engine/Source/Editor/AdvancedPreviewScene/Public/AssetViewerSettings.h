// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/SoftObjectPtr.h"
#include "UObject/ObjectRedirector.h"
#include "Engine/Scene.h"
#include "Engine/TextureCube.h"
#include "EditorUndoClient.h"
#include "ShowFlags.h"
#include "AssetViewerSettings.generated.h"

#define UE_API ADVANCEDPREVIEWSCENE_API

UENUM()
enum class EProfileConfigVersion : uint8
{
	PreVersioning = 0,
	/** New Material M_Grid applied on shared engine profiles */
	ApplyNewGridMaterial,
	VersionPlusOne,
	LatestVersion = VersionPlusOne - 1
};

PRAGMA_DISABLE_DEPRECATION_WARNINGS // For bShowGrid
/**
* Preview scene profile settings structure.
*/
USTRUCT()
struct FPreviewSceneProfile
{
	GENERATED_BODY()

	FPreviewSceneProfile()
	{		
		bSharedProfile = false;
		bIsEngineDefaultProfile = false;
		bUseSkyLighting = true;
		bShowFloor = true;
		bShowEnvironment = true;
		bRotateLightingRig = false;
		DirectionalLightIntensity = 1.0f;
		DirectionalLightColor = FLinearColor::White;
		SkyLightIntensity = 1.0f;
		LightingRigRotation = 0.0f;
		RotationSpeed = 2.0f;
		EnvironmentIntensity = 1.0f;
		EnvironmentColor = FLinearColor(0.2f, 0.2f, 0.2f, 1.0f);
		// Set up default cube map texture from editor/engine textures
		EnvironmentCubeMap = nullptr;
		EnvironmentCubeMapPath = TEXT("/Engine/EditorMaterials/AssetViewer/EpicQuadPanorama_CC+EV1.EpicQuadPanorama_CC+EV1");
		EnvironmentFloorMesh = nullptr;
		EnvironmentFloorMeshPath = TEXT("/Engine/EditorMeshes/AssetViewer/Floor_Mesh.Floor_Mesh");
		EnvironmentFloorMaterial = nullptr;
		EnvironmentFloorMaterialPath = TEXT("/Engine/EngineMaterials/M_Grid.M_Grid");
		EnvironmentFloorScale = FVector(4.0f, 4.0f, 1.0f);
		EnvironmentFloorRotation = FRotator();
		bPostProcessingEnabled = true;
		DirectionalLightRotation = FRotator(-40.f, -67.5f, 0.f);
		bEnableToneMapping = true;
		bShowMeshEdges = false;
		// This will stop scene from becoming bright due to exposure
		PostProcessingSettings.bOverride_AutoExposureMinBrightness = true;
		PostProcessingSettings.AutoExposureMinBrightness = -1;
		PostProcessingSettings.bOverride_AutoExposureMaxBrightness = true;
		PostProcessingSettings.AutoExposureMaxBrightness = 1;
	}

	/** Name to identify the profile */
	UPROPERTY(EditAnywhere, config, Category=Profile)
	FString ProfileName;

	/** Whether or not this profile should be stored in the Project ini file */
	UPROPERTY(EditAnywhere, config, Category = Profile)
	bool bSharedProfile;
	
	/** Whether or not this profile is one of the default profiles included with the engine */
	UPROPERTY(config)
	bool bIsEngineDefaultProfile;
	
	/** Whether or not image based lighting is enabled for the environment cube map */
	UPROPERTY(EditAnywhere, AdvancedDisplay, config, Category = Lighting)
	bool bUseSkyLighting;

	/** Manually set the directional light intensity (0.0 - 20.0) */
	UPROPERTY(EditAnywhere, config, Category = Lighting, meta = (UIMin = "0.0", UIMax = "20.0"))
	float DirectionalLightIntensity;

	/** Manually set the directional light colour */
	UPROPERTY(EditAnywhere, config, Category = Lighting)
	FLinearColor DirectionalLightColor;
	
	/** Manually set the sky light intensity (0.0 - 20.0) */
	UPROPERTY(EditAnywhere, config, Category = Lighting, meta = (UIMin = "0.0", UIMax = "20.0"))
	float SkyLightIntensity;

	/** Toggle rotating of the sky and directional lighting, press K and drag for manual rotating of Sky and L for Directional lighting */
	UPROPERTY(EditAnywhere, config, Category = Lighting, meta = (DisplayName = "Rotate Sky and Directional Lighting"))
	bool bRotateLightingRig;
	
	/** Toggle visibility of the environment sphere */
	UPROPERTY(EditAnywhere, config, Category = Environment)
	bool bShowEnvironment;

	/** Toggle visibility of the floor mesh */
	UPROPERTY(EditAnywhere, config, Category = Environment, meta=(DisplayAfter="EnvironmentCubeMap"))
	bool bShowFloor;

	/** Toggle visibility of floor grid on/off */
	UE_DEPRECATED(5.8, "Removed from preview scene profile since it's not relevant to every viewport")
	UPROPERTY(config)
	bool bShowGrid = false;

	/** The environment color, used if Show Environment is false. */
	UPROPERTY(EditAnywhere, config, Category = Environment, meta=(EditCondition="!bShowEnvironment"))
	FLinearColor EnvironmentColor;
	
	/** The environment intensity (0.0 - 20.0), used if Show Environment is false. */
	UPROPERTY(EditAnywhere, config, Category = Lighting, meta = (UIMin = "0.0", UIMax = "20.0", EditCondition="!bShowEnvironment"))
	float EnvironmentIntensity;

	/** Sets environment cube map used for sky lighting and reflections */
	UPROPERTY(EditAnywhere, transient, Category = Environment, meta=(EditCondition="bShowEnvironment", AssetViewInitialViewType=1, AssetViewInitialThumbnailSize=4, AssetViewSettingsName="", ForceShowPluginContent, ForceShowEngineContent))
	TSoftObjectPtr<UTextureCube> EnvironmentCubeMap;

	/** Storing path to environment cube to prevent it from getting cooked */
	UPROPERTY(config)
	FString EnvironmentCubeMapPath;

	/** Sets the environment floor mesh */
	UPROPERTY(EditAnywhere, transient, Category = Environment, meta=(EditCondition="bShowFloor", AssetViewInitialViewType=1, AssetViewInitialThumbnailSize=4, AssetViewSettingsName="", ForceShowPluginContent, ForceShowEngineContent))
	TSoftObjectPtr<UStaticMesh> EnvironmentFloorMesh;

	/** Storing path to the environment floor mesh to prevent it from getting cooked */
	UPROPERTY(config)
	FString EnvironmentFloorMeshPath;

	/** Sets the rotation of the floor mesh */
	UPROPERTY(EditAnywhere, config, Category = Environment, meta=(EditCondition="bShowFloor"))
	FRotator EnvironmentFloorRotation;

	/** Sets the scale of the floor mesh */
	UPROPERTY(EditAnywhere, config, Category = Environment, meta=(EditCondition="bShowFloor"))
	FVector EnvironmentFloorScale;

	/** Sets the environment floor material */
	UPROPERTY(EditAnywhere, transient, Category = Environment, meta=(EditCondition="bShowFloor", AssetViewInitialViewType=1, AssetViewInitialThumbnailSize=4, AssetViewSettingsName="", ForceShowPluginContent, ForceShowEngineContent))
	TSoftObjectPtr<UMaterialInterface> EnvironmentFloorMaterial;

	/** Storing path to the environment floor material to prevent it from getting cooked */
	UPROPERTY(config)
	FString EnvironmentFloorMaterialPath;

	/** Whether or not the Post Processing should influence the scene */
	UPROPERTY(EditAnywhere, config, Category = PostProcessing, AdvancedDisplay)
	bool bPostProcessingEnabled;

	/** Manual set post processing settings */
	UPROPERTY(EditAnywhere, config, Category = PostProcessing, AdvancedDisplay, meta=(ShowOnlyInnerProperties))
	FPostProcessSettings PostProcessingSettings;

	/** Current rotation value of the sky in degrees (0 - 360) */
	UPROPERTY(EditAnywhere, config, Category = Lighting, meta = (UIMin = "0", UIMax = "360"), AdvancedDisplay)
	float LightingRigRotation;

	/** Speed at which the sky rotates when rotating is toggled */
	UPROPERTY(EditAnywhere, config, Category = Lighting, AdvancedDisplay)
	float RotationSpeed;

	/** Rotation for directional light */
	UPROPERTY(config)
	FRotator DirectionalLightRotation;

	/** Useful when editing in an unlit view, prevents colors from being adjusted by the tonemapping */
	UPROPERTY(EditAnywhere, config, Category = Editing)
	bool bEnableToneMapping;

	/** Show wireframes composited on top of the shaded view */
	UPROPERTY(EditAnywhere, config, Category = Editing)
	bool bShowMeshEdges;

	/** Retrieve the environment map texture using the saved path */
	void LoadEnvironmentMap()
	{
		if (EnvironmentCubeMap == nullptr)
		{
			if (!EnvironmentCubeMapPath.IsEmpty())
			{
				// Load cube map from stored path
				UObject* LoadedObject = LoadObject<UObject>(nullptr, *EnvironmentCubeMapPath);
				while (UObjectRedirector* Redirector = Cast<UObjectRedirector>(LoadedObject))
				{
					LoadedObject = Redirector->DestinationObject;
				}

				EnvironmentCubeMap = Cast<UTextureCube>(LoadedObject);
			}
		}
	}

	void SetShowFlags(FEngineShowFlags& ShowFlags) const
	{
		// for reasons I haven't been able to discern exactly, this must be called prior to EnableAdvancedFeatures()
		// to prevent a crash in the renderer caused by unallocated or missing resources
		ShowFlags.DisableAdvancedFeatures();
		
		if (bPostProcessingEnabled)
		{
			ShowFlags.EnableAdvancedFeatures();
			ShowFlags.SetBloom(true); // bloom not included in EnableAdvancedFeatures() for thumbnails (see func comments)
		}
		else
		{
			ShowFlags.DisableAdvancedFeatures();
			ShowFlags.SetBloom(false);
		}

		ShowFlags.SetTonemapper(bEnableToneMapping);
		ShowFlags.SetMeshEdges(bShowMeshEdges);
	}
	
	/** Loads and returns the environment floor mesh */
	UE_API UStaticMesh* GetEnvironmentFloorMesh() const;
	
	/** Loads and returns the environment floor material */
	UE_API UMaterialInterface* GetEnvironmentFloorMaterial() const;
	
	/** Loads required objects for this profile */
	UE_API void LoadProfileObjects();
	
	/** Called when a profile property has been changed */
	void PostEditChangeProperty(FPropertyChangedEvent& InEvent);
	
	bool operator==(const FPreviewSceneProfile& InOther) const
	{
		return ProfileName.Equals(InOther.ProfileName);
	}

	bool operator!=(const FPreviewSceneProfile& InOther) const
	{
		return !(*this == InOther);
	}

	friend uint32 GetTypeHash(const FPreviewSceneProfile& InItem)
	{
		return GetTypeHash(InItem.ProfileName);
	}
};

PRAGMA_ENABLE_DEPRECATION_WARNINGS // For bShowGrid

UCLASS(MinimalAPI)
class UDefaultEditorProfiles : public UObject
{
	GENERATED_BODY()
public:

	UDefaultEditorProfiles()
	{
		FPreviewSceneProfile DefaultProfile;
		DefaultProfile.bIsEngineDefaultProfile = true;
		DefaultProfile.bSharedProfile = true;
		DefaultProfile.ProfileName = DefaultProfileName.ToString();
		Profiles.Add(DefaultProfile);
		
		FPreviewSceneProfile EditingProfile;
		EditingProfile.ProfileName = EditingProfileName.ToString();
		EditingProfile.bIsEngineDefaultProfile = true;
		EditingProfile.bSharedProfile = true;
		EditingProfile.bShowEnvironment = false;
		EditingProfile.bShowFloor = false;
		EditingProfile.EnvironmentColor = FLinearColor::MakeFromHSV8(0,0,10);
		EditingProfile.bUseSkyLighting = true;
		EditingProfile.bPostProcessingEnabled = false;
		EditingProfile.bShowMeshEdges = true;
		EditingProfile.bEnableToneMapping = false;
		Profiles.Add(EditingProfile);

		FPreviewSceneProfile GreyAmbientProfile;
		GreyAmbientProfile.ProfileName = GreyAmbientProfileName.ToString();
		GreyAmbientProfile.bIsEngineDefaultProfile = true;
		GreyAmbientProfile.bSharedProfile = true;
		GreyAmbientProfile.bShowEnvironment = true;
		GreyAmbientProfile.bShowFloor = true;
		GreyAmbientProfile.bUseSkyLighting = true;
		GreyAmbientProfile.bPostProcessingEnabled = false;
		GreyAmbientProfile.bShowMeshEdges = false;
		GreyAmbientProfile.bEnableToneMapping = false;
		GreyAmbientProfile.DirectionalLightIntensity = 4.0;
		GreyAmbientProfile.SkyLightIntensity = 2.0;
		GreyAmbientProfile.EnvironmentCubeMapPath = TEXT("/Engine/EditorMaterials/AssetViewer/T_GreyAmbient");
		Profiles.Add(GreyAmbientProfile);
	}

	UE_API const FPreviewSceneProfile* GetProfile(const FString& ProfileName);

	static UE_API FName DefaultProfileName;
	static UE_API FName EditingProfileName;
	static UE_API FName GreyAmbientProfileName;
	
	/** Collection of default engine-provided profiles used in various editing environments*/
	UPROPERTY()
	TArray<FPreviewSceneProfile> Profiles;
};

UCLASS(config = Editor)
class ULocalProfiles : public UObject
{
	GENERATED_BODY()
public:
	/** Collection of local scene profiles */
	UPROPERTY(config)
	TArray<FPreviewSceneProfile> Profiles;
};

UCLASS(config = Editor, defaultconfig )
class USharedProfiles : public UObject
{
	GENERATED_BODY()
public:
	EProfileConfigVersion GetVersion() const;
	void SetVersionToLatest();

	/** Collection of shared scene profiles */
	UPROPERTY(config)
	TArray<FPreviewSceneProfile> Profiles;

private:
	/** Used for auto migrations when new changes are introduced */
	UPROPERTY(Config)
	EProfileConfigVersion Version = EProfileConfigVersion::PreVersioning;
};

/**
* Default asset viewer settings.
*/
UCLASS(MinimalAPI, config=Editor, meta=(DisplayName = "Asset Viewer"))
class UAssetViewerSettings : public UObject, public FEditorUndoClient
{
	GENERATED_BODY()
public:
	UE_API UAssetViewerSettings();
	UE_API virtual ~UAssetViewerSettings() override;

	static UE_API UAssetViewerSettings* Get();

	static UE_API FPreviewSceneProfile& GetCurrentUserProjectProfile();
	
	/**
	 * Saves the config data out to the ini files
	 * @param bWarnIfFail Should we log a warning if a ini file couldn't be saved.
	 */
	UE_API void Save(bool bWarnIfFail = true);

	DECLARE_EVENT_OneParam(UAssetViewerSettings, FOnAssetViewerSettingsChangedEvent, const FName&);
	FOnAssetViewerSettingsChangedEvent& OnAssetViewerSettingsChanged() { return OnAssetViewerSettingsChangedEvent; }

	DECLARE_EVENT(UAssetViewerSettings, FOnAssetViewerProfileAddRemovedEvent);
	FOnAssetViewerProfileAddRemovedEvent& OnAssetViewerProfileAddRemoved() { return OnAssetViewerProfileAddRemovedEvent; }

	DECLARE_EVENT(UAssetViewerSettings, FOnAssetViewerSettingsPostUndo);
	FOnAssetViewerSettingsPostUndo& OnAssetViewerSettingsPostUndo() { return OnAssetViewerSettingsPostUndoEvent; }

	/** Begin UObject */
	UE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& InPropertyChangedEvent) override;
	UE_API virtual void PostEditChangeChainProperty(struct FPropertyChangedChainEvent& InPropertyChangedEvent) override;
	UE_API virtual void PostInitProperties() override;
	/** End UObject */

	/** Begin FEditorUndoClient */
	UE_API virtual void PostUndo(bool bSuccess) override;
	UE_API virtual void PostRedo(bool bSuccess) override;
	/** End FEditorUndoClient */

	/** Collection of scene profiles */
	UPROPERTY(EditAnywhere, transient, Category = Settings, meta=(ShowOnlyInnerProperties))
	TArray<FPreviewSceneProfile> Profiles;

	/** Cached value to determine whether or not a profile was added or removed */
	int32 NumProfiles;
protected:
	/** Broadcasts after an scene profile was added or deleted from the asset viewer singleton instance */
	FOnAssetViewerSettingsChangedEvent OnAssetViewerSettingsChangedEvent;

	FOnAssetViewerProfileAddRemovedEvent OnAssetViewerProfileAddRemovedEvent;

	FOnAssetViewerSettingsPostUndo OnAssetViewerSettingsPostUndoEvent;

	// This will enforce mutable CDO of UAssetViewerSettings transacted
	UPROPERTY(Config)
	bool bFakeConfigValue_HACK;
};

#undef UE_API
