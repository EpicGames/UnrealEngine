// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IPreviewProfileController.h"

#define UE_API ADVANCEDPREVIEWSCENE_API

class UAssetViewerSettings;
class UEditorPerProjectUserSettings;

/**
 * Controls the access to the preview profiles. It serves as a bridge between the AdvancedPreviewScene
 * and the UnrealEd modules and enable UnrealEd to change or observe change to the active profile.
 * 
 * @note This class was created to decouple UnrealEd from AdvancedPreviewScene and prevent
 *       circular dependencies between the modules.
 */
class FPreviewProfileController : public IPreviewProfileController
{
public:
	UE_API FPreviewProfileController();
	UE_API virtual ~FPreviewProfileController();

	/** Returns the list of available preview profiles names. */
	UE_API virtual TArray<FString> GetPreviewProfiles(int32& OutCurrentProfileIndex) const override;

	/** Set the specified preview profiles as the active one. */
	UE_API virtual bool SetActiveProfile(const FString& ProfileName) override;

	/** Returns the preview profiles currently active. */
	UE_API virtual FString GetActiveProfile() const override;

	/** Returns true if user has added one or more of their own profiles */
	UE_API virtual bool HasAnyUserProfiles() const override;

	/** Invoked after the list of available profiles has changed. */
	virtual FOnPreviewProfileListChanged& OnPreviewProfileListChanged() override { return OnPreviewProfileListChangedDelegate; }

	/** Invoked after the active preview profile changed. */
	virtual FOnPreviewProfileChanged& OnPreviewProfileChanged() override { return OnPreviewSettingChangedDelegate; }

private:
	UE_API void UpdateAssetViewerProfiles();
	UE_API void EnsureProfilesStateCoherence() const;

private:
	/** Provides the list of available preview profiles along with the profile change delegates (to update the profile combo box selection) */
	UAssetViewerSettings* AssetViewerSettings = nullptr;

	/** Holds the asset viewer profile currently used by the project. */
	UEditorPerProjectUserSettings* PerProjectSettings = nullptr;

	/** The list of available profiles. */
	TArray<FString> AssetViewerProfileNames;

	/** Holds the current profile index in the list. This is kept consistent with the cache list of names. */
	int32 CurrentProfileIndex = 0;

	FOnPreviewProfileListChanged OnPreviewProfileListChangedDelegate;
	FOnPreviewProfileChanged OnPreviewSettingChangedDelegate;

	FDelegateHandle AssetViewerSettingsProfileAddRemoveHandle;
	FDelegateHandle AssetViewerSettingsChangedHandle;
};

#undef UE_API
