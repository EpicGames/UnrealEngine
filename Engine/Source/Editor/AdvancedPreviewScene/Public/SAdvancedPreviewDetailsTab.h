// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "SlateFwd.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "PropertyEditorDelegates.h"
#include "AdvancedPreviewSceneModule.h"
#include "AssetViewerSettings.h"

#define UE_API ADVANCEDPREVIEWSCENE_API

class FAdvancedPreviewScene;
class IDetailsView;
class UAssetViewerSettings;
class UEditorPerProjectUserSettings;

class SAdvancedPreviewDetailsTab : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAdvancedPreviewDetailsTab)
		: _AdditionalSettings(nullptr)
	{}

	/** Additional settings object to display in the view */
	SLATE_ARGUMENT(UObject*, AdditionalSettings)

	/** Customizations to use for this details tab */
	SLATE_ARGUMENT(TArray<FAdvancedPreviewSceneModule::FDetailCustomizationInfo>, DetailCustomizations)

	/** Customizations to use for this details tab */
	SLATE_ARGUMENT(TArray<FAdvancedPreviewSceneModule::FPropertyTypeCustomizationInfo>, PropertyTypeCustomizations)

	/** Delegates to use for this details tab */
	SLATE_ARGUMENT(TArray<FAdvancedPreviewSceneModule::FDetailDelegates>, Delegates)

	SLATE_END_ARGS()

	UE_API SAdvancedPreviewDetailsTab();
	UE_API virtual ~SAdvancedPreviewDetailsTab() override;

	/** SWidget functions */
	UE_API void Construct(const FArguments& InArgs, const TSharedRef<FAdvancedPreviewScene>& InPreviewScene);

	UE_API void Refresh();

protected:
	UE_API void CreateSettingsView();
	UE_API void ComboBoxSelectionChanged(TSharedPtr<FString> NewSelection, ESelectInfo::Type /*SelectInfo*/);
	UE_API void UpdateSettingsView();
	UE_API void UpdateProfileNames();
	UE_API FReply AddProfileButtonClick();
	UE_API FReply CloneProfileButtonClick();
	UE_API FReply RemoveOrResetProfileButtonClick();

	UE_API void OnAssetViewerSettingsRefresh(const FName& InPropertyName);
	UE_API void OnAssetViewerSettingsPostUndo();

	/** Property viewing widget */
	TSharedPtr<IDetailsView> SettingsView;
	TSharedPtr<STextComboBox> ProfileComboBox;
	TWeakPtr<FAdvancedPreviewScene> PreviewScenePtr;
	UAssetViewerSettings* DefaultSettings;
	UObject* AdditionalSettings;

	TArray<TSharedPtr<FString>> ProfileNames;
	int32 ProfileIndex;

	UE_API void OnPreviewSceneChanged(TSharedRef<FAdvancedPreviewScene> PreviewScene);

	FDelegateHandle RefreshDelegate;
	FDelegateHandle AddRemoveProfileDelegate;
	FDelegateHandle PostUndoDelegate;

	UEditorPerProjectUserSettings* PerProjectSettings;

	TArray<FAdvancedPreviewSceneModule::FDetailCustomizationInfo> DetailCustomizations;

	TArray<FAdvancedPreviewSceneModule::FPropertyTypeCustomizationInfo> PropertyTypeCustomizations;

	TArray<FAdvancedPreviewSceneModule::FDetailDelegates> Delegates;
};

#undef UE_API
