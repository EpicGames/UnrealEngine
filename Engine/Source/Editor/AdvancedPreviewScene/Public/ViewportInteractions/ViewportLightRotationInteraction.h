// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ViewportInteractions/ViewportDragInteraction.h"
#include "ViewportInteractions/ViewportInteractionBindings.h"

#include "ViewportLightRotationInteraction.generated.h"

class FAdvancedPreviewScene;
class IToolsContextRenderAPI;

/**
 * Viewport interaction that rotates the directional light.
 * Used in editors with an FAdvancedPreviewScene (e.g., Static Mesh Editor, Persona).
 */
UCLASS(MinimalAPI, Transient)
class UViewportLightRotationInteraction : public UViewportDragInteraction
{
	GENERATED_BODY()

public:
	ADVANCEDPREVIEWSCENE_API UViewportLightRotationInteraction();

	virtual void Render(IToolsContextRenderAPI* InRenderAPI) override;

	virtual bool CanBeActivated() const override;
	virtual void OnDrag(const FDragArgs& InDrag) override;
	virtual void OnDragDelta(float InMouseDeltaX, float InMouseDeltaY) override;

private:
	FVector2D ScreenPosition;

	FAdvancedPreviewScene* GetAdvancedPreviewScene() const;
};
