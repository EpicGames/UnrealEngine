// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ViewportInteractions/ViewportDragInteraction.h"

#include "ViewportSkyRotationInteraction.generated.h"

class FAdvancedPreviewScene;

/**
 * Viewport interaction that rotates the sky environment.
 * Used in editors with an FAdvancedPreviewScene (e.g., Static Mesh Editor, Persona).
 */
UCLASS(MinimalAPI, Transient)
class UViewportSkyRotationInteraction : public UViewportDragInteraction
{
	GENERATED_BODY()

public:
	ADVANCEDPREVIEWSCENE_API UViewportSkyRotationInteraction();

	virtual bool CanBeActivated() const override;
	virtual void OnDragDelta(float InMouseDeltaX, float InMouseDeltaY) override;

private:
	FAdvancedPreviewScene* GetAdvancedPreviewScene() const;
};
